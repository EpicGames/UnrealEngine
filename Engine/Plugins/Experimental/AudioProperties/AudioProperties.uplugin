{
	"FileVersion": 3,
	"Version": 0,
	"VersionName": "0.0",
	"FriendlyName": "Audio Properties",
	"Description": "Allows to define arbitrary derivable sets of properties to be injected in any audio asset",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AudioProperties",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AudioPropertiesEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
	]
}