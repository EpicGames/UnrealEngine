// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AudioProperties : ModuleRules
	{
		public AudioProperties(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"AudioExtensions",
					"AssetRegistry",
					"Core",
					"CoreUObject",
					"TargetPlatform"
				}
			);
		}
	}
}