// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesModule.h"
#include "Modules/ModuleManager.h"

void FAudioPropertiesModule::StartupModule()
{
}

void FAudioPropertiesModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FAudioPropertiesModule, AudioProperties);