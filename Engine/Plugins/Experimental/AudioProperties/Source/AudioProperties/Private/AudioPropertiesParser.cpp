// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesParser.h"

#include "AssetRegistry/AssetData.h"
#include "AudioPropertiesBindings.h"
#include "AudioPropertiesLogs.h"
#include "AudioPropertiesParsingData.h"
#include "AudioPropertiesSheet.h"
#include "AudioPropertiesUtils.h"
#include "UObject/PropertyAccessUtil.h"
#include "UObject/UnrealTypePrivate.h"

namespace AudioPropertiesParserPrivate
{
	static const FName ClampMinTagName = "ClampMin";
	static const FName ClampMaxTagName = "ClampMax";
	static const FName UIMinTagName = "UIMin";
	static const FName UIMaxTagName = "UIMax";
	static const FName ValidEnumValuesTagName = "ValidEnumValues";
	static const FName InvalidEnumValuesTagName = "InvalidEnumValues";

	bool IsPropertyLocallyIgnored(UObject& TargetObject, FProperty& TargetProperty)
	{
#if WITH_EDITORONLY_DATA
		const IAudioPropertiesSheetAssetUserInterface* AsPropertySheetUser = Cast<IAudioPropertiesSheetAssetUserInterface>(&TargetObject);

		if (AsPropertySheetUser)
		{
			return !AsPropertySheetUser->ShouldParseProperty(TargetProperty);	
		}
#endif
		return false;
	}

	template< typename NumericType>
	NumericType ClampByPropertyMetadata(const FProperty& TargetProperty, NumericType InValueToClamp)
	{
		static_assert(TIsArithmetic<NumericType>::Value, "Property to clamp must have numeric type");

		const FString& MetaClampMinString = TargetProperty.GetMetaData(ClampMinTagName);
		const FString& MetaClampMaxString = TargetProperty.GetMetaData(ClampMaxTagName);
		const FString& MetaUIMinString = TargetProperty.GetMetaData(UIMinTagName);
		const FString& MetaUIMaxString = TargetProperty.GetMetaData(UIMaxTagName);

 		const bool bShouldClamp = !MetaClampMinString.IsEmpty() || !MetaClampMaxString.IsEmpty() || !MetaUIMinString.IsEmpty() || !MetaUIMaxString.IsEmpty();
 
 		if (!bShouldClamp)
 		{
 			return MoveTemp(InValueToClamp);
 		}
		
		NumericType ClampMin = TNumericLimits<NumericType>::Lowest();
		if (!MetaClampMinString.IsEmpty())
		{
			TTypeFromString<NumericType>::FromString(ClampMin, *MetaClampMinString);
		}

		NumericType ClampMax = TNumericLimits<NumericType>::Max();
		if (!MetaClampMaxString.IsEmpty())
		{
			TTypeFromString<NumericType>::FromString(ClampMax, *MetaClampMaxString);
		}
		
		NumericType UIMin = TNumericLimits<NumericType>::Lowest();
		if (!MetaUIMinString.IsEmpty())
		{
			TTypeFromString<NumericType>::FromString(UIMin, *MetaUIMinString);
		}
		
		NumericType UIMax = TNumericLimits<NumericType>::Max();
		if (!MetaUIMaxString.IsEmpty())
		{
			TTypeFromString<NumericType>::FromString(UIMax, *MetaUIMaxString);
		}

		const NumericType MinAllowedValue = FMath::Max(UIMin, ClampMin);
		const NumericType MaxAllowedValue = FMath::Min(UIMax, ClampMax); 
		const NumericType ClampedValue = FMath::Clamp<NumericType>(InValueToClamp, MinAllowedValue, MaxAllowedValue);
		
		return ClampedValue;
	}

	bool ShouldApplyTargetPropertyMetadata(const FProperty& TargetProperty, const FPropertyBagPropertyDesc& SourcePropertyDesc, const FInstancedPropertyBag& PropertiesToInject)
	{
		bool bShouldApplyMetadata = false;

		if (!SourcePropertyDesc.ContainerTypes.IsEmpty())
		{
			return bShouldApplyMetadata;
		}
		
		const FName TargetPropertyName = SourcePropertyDesc.CachedProperty->GetFName();

		switch (SourcePropertyDesc.ValueType)
		{
		case EPropertyBagPropertyType::Float:
		{
			const float SourceValue = PropertiesToInject.GetValueFloat(TargetPropertyName).GetValue();
			const float ClampedFloatValue = AudioPropertiesParserPrivate::ClampByPropertyMetadata(TargetProperty, SourceValue);
			bShouldApplyMetadata = SourceValue != ClampedFloatValue;
		}
		break;
		case EPropertyBagPropertyType::Double:
		{
			const float SourceValue = PropertiesToInject.GetValueDouble(TargetPropertyName).GetValue();
			const double ClampedDoubleValue = AudioPropertiesParserPrivate::ClampByPropertyMetadata(TargetProperty, SourceValue);
			bShouldApplyMetadata = SourceValue != ClampedDoubleValue;
		}
		break;
		case EPropertyBagPropertyType::Int32:
		{
			const float SourceValue = PropertiesToInject.GetValueInt32(TargetPropertyName).GetValue();
			const int32 ClampedInt32Value = AudioPropertiesParserPrivate::ClampByPropertyMetadata(TargetProperty, SourceValue);
			bShouldApplyMetadata = SourceValue != ClampedInt32Value;
		}
		break;
		case EPropertyBagPropertyType::Int64:
		{
			const float SourceValue = PropertiesToInject.GetValueInt64(TargetPropertyName).GetValue();
			const int64 ClampedInt64Value = AudioPropertiesParserPrivate::ClampByPropertyMetadata(TargetProperty, PropertiesToInject.GetValueInt64(TargetPropertyName).GetValue());
			bShouldApplyMetadata = SourceValue != ClampedInt64Value;
		}
		break;
		default:
			break;
		}

		if (bShouldApplyMetadata)
		{
			UE_LOGF(LogAudioProperties, Log, "Sheet Property '%ls' needs to be clamped to be properly applied to Target Property '%ls'", *TargetPropertyName.ToString(), *TargetProperty.GetFName().ToString())	
		}

		return bShouldApplyMetadata;
	}

	bool ApplyTargetPropertyMetadata(const FProperty& TargetProperty, const FPropertyBagPropertyDesc& SourcePropertyDesc, FInstancedPropertyBag& PropertiesToInject)
	{
		bool bShouldApplyMetadata = false;

		if (!SourcePropertyDesc.ContainerTypes.IsEmpty())
		{
			return bShouldApplyMetadata;
		}
		
		const FName TargetPropertyName = SourcePropertyDesc.CachedProperty->GetFName();

		switch (SourcePropertyDesc.ValueType)
		{
		case EPropertyBagPropertyType::Float:
		{
			const float SourceValue = PropertiesToInject.GetValueFloat(TargetPropertyName).GetValue();
			const float ClampedFloatValue = AudioPropertiesParserPrivate::ClampByPropertyMetadata(TargetProperty, SourceValue);
			bShouldApplyMetadata = SourceValue != ClampedFloatValue;
			PropertiesToInject.SetValueFloat(TargetPropertyName, ClampedFloatValue);
		}
		break;
		case EPropertyBagPropertyType::Double:
		{
			const float SourceValue = PropertiesToInject.GetValueDouble(TargetPropertyName).GetValue();
			const double ClampedDoubleValue = AudioPropertiesParserPrivate::ClampByPropertyMetadata(TargetProperty, SourceValue);
			bShouldApplyMetadata = SourceValue != ClampedDoubleValue;
			PropertiesToInject.SetValueDouble(TargetPropertyName, ClampedDoubleValue);
		}
		break;
		case EPropertyBagPropertyType::Int32:
		{
			const float SourceValue = PropertiesToInject.GetValueInt32(TargetPropertyName).GetValue();
			const int32 ClampedInt32Value = AudioPropertiesParserPrivate::ClampByPropertyMetadata(TargetProperty, SourceValue);
			bShouldApplyMetadata = SourceValue != ClampedInt32Value;
			PropertiesToInject.SetValueInt32(TargetPropertyName, ClampedInt32Value);
		}
		break;
		case EPropertyBagPropertyType::Int64:
		{
			const float SourceValue = PropertiesToInject.GetValueInt64(TargetPropertyName).GetValue();
			const int64 ClampedInt64Value = AudioPropertiesParserPrivate::ClampByPropertyMetadata(TargetProperty, PropertiesToInject.GetValueInt64(TargetPropertyName).GetValue());
			bShouldApplyMetadata = SourceValue != ClampedInt64Value;
			PropertiesToInject.SetValueInt64(TargetPropertyName, ClampedInt64Value);
		}
		break;
		default:
			break;
		}

		return bShouldApplyMetadata;
	}

	const bool IsValidEnumForTargetMetadata(const FProperty& TargetProperty, const FPropertyBagPropertyDesc& SourcePropertyDesc, const FInstancedPropertyBag& PropertyToInjectParentBag)
	{
		if (SourcePropertyDesc.ValueType == EPropertyBagPropertyType::Enum && TargetProperty.HasMetaData(ValidEnumValuesTagName))
		{
			check(SourcePropertyDesc.CachedProperty)
			TArray<FName> ValidEnumValues;
			const FString EnumValueToInject = PropertyToInjectParentBag.GetValueSerializedString(SourcePropertyDesc.CachedProperty->GetFName()).GetValue();
			TArray<FString> ValidEnumValuesAsString;
			TArray<FString> InvalidEnumValuesAsString;
			TargetProperty.GetMetaData(ValidEnumValuesTagName).ParseIntoArray(ValidEnumValuesAsString, TEXT(","));
			TargetProperty.GetMetaData(InvalidEnumValuesTagName).ParseIntoArray(InvalidEnumValuesAsString, TEXT(","));


			for (FString& Value : ValidEnumValuesAsString)
			{
				Value.TrimStartInline();
			}

			for (FString& Value : InvalidEnumValuesAsString)
			{
				Value.TrimStartInline();
			}
			
			return (ValidEnumValuesAsString.IsEmpty() || ValidEnumValuesAsString.Contains(EnumValueToInject)) && !InvalidEnumValuesAsString.Contains(EnumValueToInject);
		}

		return true;
	}

	const bool ShouldInjectProperty(const FProperty& TargetProperty, const FPropertyBagPropertyDesc& SourcePropertyDesc, const FInstancedPropertyBag& PropertyToInjectParentBag)
	{
		check(SourcePropertyDesc.CachedProperty)
		bool bShouldInject = true;
		
		bShouldInject &= PropertyAccessUtil::ArePropertiesCompatible(&TargetProperty, SourcePropertyDesc.CachedProperty);
		bShouldInject &= IsValidEnumForTargetMetadata(TargetProperty, SourcePropertyDesc, PropertyToInjectParentBag);
		bShouldInject &= TargetProperty.HasAllPropertyFlags(CPF_Edit) && !TargetProperty.HasAnyPropertyFlags(CPF_EditConst);
		
		return bShouldInject;
	}

#if WITH_EDITOR
	void VisitLeafMostPropertiesNonConst(UAudioPropertiesSheetAsset& InSheetAsset, TFunction<void(FPropertyBagPropertyDesc& /*Property Desc */, UAudioPropertiesSheetAsset& /*Owning Sheet Asset*/)> OnPropertyVisited)
	{
		TObjectPtr<UAudioPropertiesSheetAsset> SourceSheet = &InSheetAsset;
		const FAudioPropertiesSheet& LeafPropertiesSheet = InSheetAsset.PropertiesSheet;

		while (SourceSheet != nullptr)
		{
			const FInstancedPropertyBag& SheetProperties = SourceSheet->PropertiesSheet.Properties;

			if (const UPropertyBag* BagStruct = SheetProperties.GetPropertyBagStruct())
			{
				for (const FPropertyBagPropertyDesc& ConstPropertyDesc : BagStruct->GetPropertyDescs())
				{
					const bool bSkipOverridenProperty = SourceSheet != &InSheetAsset && LeafPropertiesSheet.IsPropertyOverridden(*ConstPropertyDesc.CachedProperty);

					if (!bSkipOverridenProperty)
					{
						FPropertyBagPropertyDesc& PropertyDesc = const_cast<FPropertyBagPropertyDesc&>(ConstPropertyDesc);
						OnPropertyVisited(PropertyDesc, *SourceSheet);
					}

				}
			}

			const UAudioPropertiesSheetAsset* ParentPtr = SourceSheet->PropertiesSheet.Parent;
			SourceSheet = const_cast<UAudioPropertiesSheetAsset*>(ParentPtr);
		}
	}
#endif
}


void FAudioPropertiesParser::GenerateNameMatchedParsingData(UE::AudioGameplay::FAudioPropertiesParsingData& ParsingData)
{
	if (!ensureMsgf(ParsingData.TargetClass, TEXT("Trying to generate name matching parsing data with an empty class, returning")))
	{
		return;
	}

	for (TFieldIterator<FProperty> PropertyIt(ParsingData.TargetClass); PropertyIt; ++PropertyIt)
	{
		FProperty* TargetProperty = *PropertyIt;
		
		if (TargetProperty)
		{
			const FAudioPropertiesSheet* SourceSheet = ParsingData.SourcePropertySheet;
			if (SourceSheet)
			{

				auto OnLeafPropertyVisited = [InParsingData = &ParsingData, TargetProperty](const FPropertyBagPropertyDesc& PropertyDesc, const FAudioPropertiesSheet& OwningSheet)
				{
					const FName& PropertyName = TargetProperty->GetFName();

					if (PropertyDesc.Name == PropertyName)
					{
						const bool bShouldInjectProperty = AudioPropertiesParserPrivate::ShouldInjectProperty(*TargetProperty, PropertyDesc, OwningSheet.Properties);

						if (bShouldInjectProperty)
						{
							InParsingData->SourceToTargetPropertyBindings.Add(PropertyName, PropertyName);
						}
					}
				};

				AudioPropertiesUtils::VisitLeafMostProperties(*SourceSheet, OnLeafPropertyVisited);
			}
			else
			{
				const FName& PropertyName = TargetProperty->GetFName();
				ParsingData.SourceToTargetPropertyBindings.Add(PropertyName, PropertyName);
			}
		}
	}
}

void FAudioPropertiesParser::InjectPropertyValuesIntoObject(UObject& TargetObject, const UE::AudioGameplay::FAudioPropertiesParsingData& ParsingData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioPropertiesParser::InjectPropertyValuesIntoObject);


	if (!ensureMsgf(TargetObject.GetClass(), TEXT("Expected a valid target object class")))
	{
		return;
	}

	if (!ensureMsgf(ParsingData.TargetClass, TEXT("Expected a valid target class in parsing data")))
	{
		return;
	}

	if (!ensureMsgf(ParsingData.SourcePropertySheet, TEXT("Expected a valid source sheet in parsing data")))
	{
		return;
	}
	
	const FName& TargetObjectName = TargetObject.GetFName();

	const FAudioPropertiesSheet* LeafSheet = ParsingData.SourcePropertySheet;
	const FAudioPropertiesSheet* SourceSheet = ParsingData.SourcePropertySheet;

	if (TargetObject.GetClass() != ParsingData.TargetClass)
	{
		const FName& TargetObjectClassName = TargetObject.GetClass()->GetFName();
		const FName& ParsingDataClassName = ParsingData.TargetClass->GetFName();

		UE_LOGF(LogAudioProperties, Warning, "Trying to inject property values for object %ls of class %ls with parsing data targeting class %ls. Injection aborted", *TargetObjectName.ToString(), *TargetObjectClassName.ToString(), *ParsingDataClassName.ToString());
		return;
	}

	bool bHasInjectedAnyProperty = false;
	
	while (SourceSheet != nullptr)
	{
		FInstancedPropertyBag PropertiesToInject;
		PropertiesToInject.MigrateToNewBagInstance(SourceSheet->Properties);
		
		for (const TPair<FName, FName>& PropertyBinding : ParsingData.SourceToTargetPropertyBindings)
		{
			const FName& SourcePropertyName = PropertyBinding.Value;
			const FName& TargetPropertyName = PropertyBinding.Key;
			
			const FPropertyBagPropertyDesc* SourcePropertyDesc = PropertiesToInject.FindPropertyDescByName(SourcePropertyName);
			FProperty* TargetProperty = TargetObject.GetClass()->FindPropertyByName(TargetPropertyName);

			if (SourcePropertyDesc && TargetProperty)
			{
				if (AudioPropertiesParserPrivate::IsPropertyLocallyIgnored(TargetObject, *TargetProperty))
				{
					continue;
				}

				if (AudioPropertiesParserPrivate::ShouldInjectProperty(*TargetProperty, *SourcePropertyDesc, PropertiesToInject))
				{
					check(SourcePropertyDesc->CachedProperty)
					const bool bSkipOverridenProperty = SourceSheet != LeafSheet && LeafSheet->IsPropertyOverridden(*SourcePropertyDesc->CachedProperty);

					if (!bSkipOverridenProperty)
					{
						FConstStructView SourcePropertyBagView = PropertiesToInject.GetValue();
						const void* SourceAddr = SourcePropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>(SourcePropertyBagView.GetMemory());
						void* DestinationAddr = TargetProperty->ContainerPtrToValuePtr<void>(&TargetObject);
						const FName& PropertyName = SourcePropertyDesc->CachedProperty->GetFName();
						
						const bool bPropertiesHaveSameValue = PropertyAccessUtil::IsCompletePropertyIdentical(SourcePropertyDesc->CachedProperty, SourceAddr, TargetProperty, DestinationAddr);

						if (bPropertiesHaveSameValue)
						{
							UE_LOGF(LogAudioProperties, Verbose, "Skipping copy property %ls to object %ls - value is identical", *PropertyName.ToString(), *TargetObjectName.ToString());
							continue;
						}
						
#if WITH_EDITOR
						AudioPropertiesParserPrivate::ApplyTargetPropertyMetadata(*TargetProperty, *SourcePropertyDesc, PropertiesToInject);
#endif
 						const bool bSuccessfulCopy = PropertyAccessUtil::CopyCompletePropertyValue(SourcePropertyDesc->CachedProperty, SourceAddr, TargetProperty, DestinationAddr);
						bHasInjectedAnyProperty |= bSuccessfulCopy;

						if (!bSuccessfulCopy)
						{
							UE_LOGF(LogAudioProperties, Verbose, "Failed to copy property %ls to object %ls.", *PropertyName.ToString(), *TargetObjectName.ToString());
						}
						else
						{
							UE_LOGF(LogAudioProperties, Verbose, "Successfully copied property %ls to object %ls, marking package dirty", *PropertyName.ToString(), *TargetObjectName.ToString());
						}
					}
				}
			}
		}

		//Inject instanced object properties
		for(const UE::AudioGameplay::FAudioPropertiesParsingData& InstancedObjectParsingData : ParsingData.InstancedObjectsParsingData)
		{
			UClass* InstancedObjectClass = InstancedObjectParsingData.TargetClass;

			if (!InstancedObjectClass)
			{
				return;
			}

			for (TFieldIterator<FProperty> PropIt(TargetObject.GetClass(), EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
			{
				FProperty* Property = *PropIt;

				if(AudioPropertiesParserPrivate::IsPropertyLocallyIgnored(TargetObject, *Property))
				{
					continue;
				}

				auto IsRelevantObjectProperty = [](FObjectProperty* ObjectProperty, UClass* TargetObjectClass)-> bool
				{
					check(ObjectProperty);
					check(TargetObjectClass);
					
					const bool bIsInstanced = (ObjectProperty->PropertyFlags & CPF_InstancedReference) != 0;

					// A property might be relevant both if it is a child or a parent of our target
					// We are interested in a children class when parsing properties that might be inherited
					// We are interested in a parent when creating a new instanced object, as that could be of a child class  
					const bool bTargetPropertyIsChild = ObjectProperty->PropertyClass->IsChildOf(TargetObjectClass);
					const bool bTargetPropertyIsParent = TargetObjectClass->IsChildOf(ObjectProperty->PropertyClass);
					const bool bIsRelevantClass = bTargetPropertyIsChild || bTargetPropertyIsParent;

					return bIsInstanced && bIsRelevantClass;
				};
				
				//Injects into an instance object property of the relevant type or creates a new one if instructed
				if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
				{
					if (IsRelevantObjectProperty(ObjectProperty, InstancedObjectClass))
					{						
						void* PropertyValueAddress = ObjectProperty->ContainerPtrToValuePtr<void>(&TargetObject);
						UObject* InstancedObject = ObjectProperty->GetObjectPropertyValue(PropertyValueAddress);
								
						if (InstancedObject && InstancedObject->IsA(InstancedObjectClass))
						{
							UE_LOGF(LogAudioProperties, Verbose, "Injecting Properties to Instanced Object %ls (%ls)", *InstancedObject->GetName(), *Property->GetName());
							InjectPropertyValuesIntoObject(*InstancedObject, InstancedObjectParsingData);
						}
						
						if(!InstancedObject && InstancedObjectParsingData.bCreateNewObjects && InstancedObjectClass->IsChildOf(ObjectProperty->PropertyClass))
						{
							FString NewObjectName = FString::Printf(TEXT("%s_Instanced"), *Property->GetName());
							EObjectFlags FlagsToApply = TargetObject.GetMaskedFlags(RF_PropagateToSubObjects);
							UObject* NewInstancedObject = NewObject<UObject>(&TargetObject, InstancedObjectClass, FName(NewObjectName), FlagsToApply);

							if(NewInstancedObject)
							{
								InjectPropertyValuesIntoObject(*NewInstancedObject, InstancedObjectParsingData);
								
								ObjectProperty->SetObjectPropertyValue(PropertyValueAddress, NewInstancedObject);
								TargetObject.MarkPackageDirty();

								UE_LOGF(LogAudioProperties, Verbose, "Created new instanced object %ls (%ls)", *NewInstancedObject->GetName(), *Property->GetName());
									
							}
						}
					}
				}
				//Otherwise, check if the property is an array of instanced objects of the relevant class and does the same
				else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
				{
					if (FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(ArrayProperty->Inner))
					{						
						if (IsRelevantObjectProperty(InnerObjectProperty, InstancedObjectClass))
						{
							void* ArrayPtr = ArrayProperty->ContainerPtrToValuePtr<void>(&TargetObject);
							FScriptArrayHelper Helper(ArrayProperty, ArrayPtr);
							bool bHasInstancedObject = false;

							for (int32 Index = 0; Index < Helper.Num(); ++Index)
							{
								UObject* InstancedObject = InnerObjectProperty->GetObjectPropertyValue(Helper.GetRawPtr(Index));
								
								if (InstancedObject && InstancedObject->IsA(InstancedObjectClass))
								{
									UE_LOGF(LogAudioProperties, Verbose, "Injecting Properties to Instanced Object %ls (%ls[%d])", *InstancedObject->GetName(), *ArrayProperty->GetName(), Index);
									InjectPropertyValuesIntoObject(*InstancedObject, InstancedObjectParsingData);
									bHasInstancedObject = true;
								}
							}

							if(InstancedObjectParsingData.bCreateNewObjects && !bHasInstancedObject)
							{
								FString NewObjectName = FString::Printf(TEXT("%s_%d"), *Property->GetName(), Helper.Num());
								EObjectFlags FlagsToApply = TargetObject.GetMaskedFlags(RF_PropagateToSubObjects);
								UObject* NewInstancedObject = NewObject<UObject>(&TargetObject, InstancedObjectClass, FName(NewObjectName), FlagsToApply);

								if(NewInstancedObject)
								{
									InjectPropertyValuesIntoObject(*NewInstancedObject, InstancedObjectParsingData);

									const int32 NewIndex = Helper.AddValue();
									void* NewElementPtr = Helper.GetRawPtr(NewIndex);
									InnerObjectProperty->SetObjectPropertyValue(NewElementPtr, NewInstancedObject);
									TargetObject.MarkPackageDirty();

									UE_LOGF(LogAudioProperties, Verbose, "Created new instanced object %ls (%ls[%d])", *NewInstancedObject->GetName(), *ArrayProperty->GetName(), NewIndex);
									
								}
							}
						}
					}
				}
			}
		}
		
		SourceSheet = SourceSheet->Parent ? &SourceSheet->Parent->PropertiesSheet : nullptr;
	}

	if (bHasInjectedAnyProperty)
	{
		if (UPackage* ObjectPackage = TargetObject.GetPackage())
		{
			if (!ObjectPackage->IsDirty())
			{
				TargetObject.MarkPackageDirty();
			}
		}
	}
}

#if WITH_EDITOR

bool FAudioPropertiesParser::ShouldApplyTargetMetadataToProperties(const UE::AudioGameplay::FAudioPropertiesParsingData& ParsingData)
{
	if(!ensureMsgf(ParsingData.TargetClass, TEXT("Expected a valid target class in parsing data")))
	{
		return false;
	}

	if (!ensureMsgf(ParsingData.SourcePropertySheet, TEXT("Expected a valid source sheet in parsing data")))
	{
		return false;
	}

	bool ReturnShouldApplyMetadata = false;
	auto CheckTargetPropertyMetadata = [&ParsingData, &ReturnShouldApplyMetadata](const FPropertyBagPropertyDesc& SheetPropertyDesc, const FAudioPropertiesSheet& OwningSheet)
	{
		const FName* TargetPropertyName = ParsingData.SourceToTargetPropertyBindings.FindKey(SheetPropertyDesc.Name);

		if (!TargetPropertyName)
		{
			return;
		}

		FProperty* TargetProperty = ParsingData.TargetClass->FindPropertyByName(*TargetPropertyName);

		if (!TargetProperty)
		{
			return;
		}

		const FInstancedPropertyBag& SheetPropertyBag = OwningSheet.Properties;

		if (!PropertyAccessUtil::ArePropertiesCompatible(TargetProperty, SheetPropertyDesc.CachedProperty))
		{
			return;
		}

		FConstStructView SourcePropertyBagView = SheetPropertyBag.GetValue();
		const void* SourceAddr = SheetPropertyDesc.CachedProperty->ContainerPtrToValuePtr<void>(SourcePropertyBagView.GetMemory());
		const bool bParseNewValue = false;

		const bool bShouldApplyMetadata = AudioPropertiesParserPrivate::ShouldApplyTargetPropertyMetadata(*TargetProperty, SheetPropertyDesc, SheetPropertyBag);
		const bool bIsValidEnumForMetadata = AudioPropertiesParserPrivate::IsValidEnumForTargetMetadata(*TargetProperty, SheetPropertyDesc, SheetPropertyBag);

		if (bShouldApplyMetadata || !bIsValidEnumForMetadata)
		{
			ReturnShouldApplyMetadata = true;
		}
	};

	AudioPropertiesUtils::VisitLeafMostProperties(*ParsingData.SourcePropertySheet, CheckTargetPropertyMetadata);
	
	return ReturnShouldApplyMetadata;
}

void FAudioPropertiesParser::ApplyTargetMetadataToSheetTree(UAudioPropertiesSheetAsset& InSheetAsset, const UE::AudioGameplay::FAudioPropertiesParsingData& ParsingData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioPropertiesParser::ApplyTargetMetadataToSheetTree);


	if (!ensureMsgf(ParsingData.TargetClass, TEXT("Expected a valid target class in parsing data")))
	{
		return;
	}
	
	auto ApplyTargetPropertyMetadata = [&ParsingData](FPropertyBagPropertyDesc& SheetPropertyDesc, UAudioPropertiesSheetAsset& OwningSheet)
	{
		const FName* TargetPropertyName = ParsingData.SourceToTargetPropertyBindings.FindKey(SheetPropertyDesc.Name);

		if (!TargetPropertyName)
		{
			return;
		}

		FProperty* TargetProperty = ParsingData.TargetClass->FindPropertyByName(*TargetPropertyName);

		if (!TargetProperty)
		{
			return;
		}

		FInstancedPropertyBag& SheetPropertyBag = OwningSheet.PropertiesSheet.Properties;


		if (!AudioPropertiesParserPrivate::ShouldInjectProperty(*TargetProperty, SheetPropertyDesc, SheetPropertyBag))
		{
			return;
		}

		FConstStructView SourcePropertyBagView = SheetPropertyBag.GetValue();
		const void* SourceAddr = SheetPropertyDesc.CachedProperty->ContainerPtrToValuePtr<void>(SourcePropertyBagView.GetMemory());

		if (AudioPropertiesParserPrivate::ApplyTargetPropertyMetadata(*TargetProperty, SheetPropertyDesc, SheetPropertyBag))
		{
			if (UPackage* ObjectPackage = OwningSheet.GetPackage())
			{
				if (!ObjectPackage->IsDirty())
				{
					OwningSheet.MarkPackageDirty();
				}	
			}
		}
	};

	AudioPropertiesParserPrivate::VisitLeafMostPropertiesNonConst(InSheetAsset, ApplyTargetPropertyMetadata);
}

#endif