// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UObject; 
class UAudioPropertiesSheetAsset;
struct FAudioPropertiesSheet;

namespace UE
{
	namespace AudioGameplay
	{
		struct FAudioPropertiesParsingData;
	}
}

class FAudioPropertiesParser
{
public:
	static void GenerateNameMatchedParsingData(UE::AudioGameplay::FAudioPropertiesParsingData& ParsingData);
	static void InjectPropertyValuesIntoObject(UObject& TargetObject, const UE::AudioGameplay::FAudioPropertiesParsingData& ParsingData);
	
#if WITH_EDITOR
	static bool ShouldApplyTargetMetadataToProperties(const UE::AudioGameplay::FAudioPropertiesParsingData& ParsingData);
	static void ApplyTargetMetadataToSheetTree(UAudioPropertiesSheetAsset& InSheetAsset, const UE::AudioGameplay::FAudioPropertiesParsingData& ParsingData);
#endif
};