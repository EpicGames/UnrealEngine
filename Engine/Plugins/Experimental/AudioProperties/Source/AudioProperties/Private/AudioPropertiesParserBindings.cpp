// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesParserBindings.h"

#include "AudioPropertiesBindings.h"
#include "AudioPropertiesParser.h"
#include "AudioPropertiesParsingData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioPropertiesParserBindings)

bool UAudioPropertiesParserBindings::GenerateParsingData(TObjectPtr<const UObject> TargetObject, const FAudioPropertiesSheet& SourceSheet, UE::AudioGameplay::FAudioPropertiesParsingData& InOutData) const
{
	if (!BindingsAsset || !TargetObject)
	{
		return false;
	}
	
	InOutData.SourcePropertySheet = &SourceSheet;
	InOutData.TargetClass = TargetObject->GetClass();
	InOutData.SourceToTargetPropertyBindings = BindingsAsset->ObjectPropertyToSheetPropertyMap;
	return true;
}

bool UAudioPropertiesParserBindings::ParseProperties(TObjectPtr<UObject> TargetObject, const FAudioPropertiesSheet& PropertiesToParse) const
{
	UE::AudioGameplay::FAudioPropertiesParsingData ParsingData;
	
	if (!GenerateParsingData(TargetObject, PropertiesToParse, ParsingData))
	{
		return false;
	}

	FAudioPropertiesParser::InjectPropertyValuesIntoObject(*TargetObject, ParsingData);
	return true;
}

#if WITH_EDITOR

bool UAudioPropertiesParserBindings::GenerateParsingDataFromTargetClass(UClass* TargetClass, UE::AudioGameplay::FAudioPropertiesParsingData& InOutData) const
{
	if (!BindingsAsset || !TargetClass)
	{
		return false;
	}

	InOutData.TargetClass = TargetClass;
	InOutData.SourceToTargetPropertyBindings = BindingsAsset->ObjectPropertyToSheetPropertyMap;
	return true;
}
#endif
