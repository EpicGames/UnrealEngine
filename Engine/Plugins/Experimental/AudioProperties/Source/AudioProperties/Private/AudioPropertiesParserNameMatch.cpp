// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesParserNameMatch.h"

#include "AudioPropertiesParser.h"
#include "AudioPropertiesParsingData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioPropertiesParserNameMatch)


bool UAudioPropertiesParserNameMatch::GenerateParsingData(TObjectPtr<const UObject> TargetObject, const FAudioPropertiesSheet& SourceSheet, UE::AudioGameplay::FAudioPropertiesParsingData& InOutData) const
{
	if (!TargetObject)
	{
		return false;
	}

	
	for(const FInstancedObjectParsingInstructions& InstancedObjectParsingRules : InstancedObjectsRules)
	{
		UClass* TargetClass = InstancedObjectParsingRules.InstancedObjectClass.Get();

		if(!TargetClass)
		{
			continue;
		}
		
		UE::AudioGameplay::FAudioPropertiesParsingData InstancedObjectParsingData;
		InstancedObjectParsingData.SourcePropertySheet = &SourceSheet;
		InstancedObjectParsingData.TargetClass = TargetClass;
		InstancedObjectParsingData.bCreateNewObjects = InstancedObjectParsingRules.bCreateIfMissing;
		
		if(!InstancedObjectParsingRules.bOverrideParsing)
		{
			GenerateParsingDataFromTargetClass(TargetClass, InstancedObjectParsingData);
		}
		else
		{
			if(InstancedObjectParsingRules.PropertiesParser)
			{
				TObjectPtr<UObject> ParsingDataGenerationObject = NewObject<UObject>(GetOuter(), InstancedObjectParsingRules.InstancedObjectClass.Get());
				InstancedObjectParsingRules.PropertiesParser->GenerateParsingData(ParsingDataGenerationObject, SourceSheet, InstancedObjectParsingData);
			}
		}
		
		InOutData.InstancedObjectsParsingData.Add(MoveTemp(InstancedObjectParsingData));	
	}

	InOutData.SourcePropertySheet = &SourceSheet;
	UClass* ObjectClass = TargetObject->GetClass();
	
	return GenerateParsingDataFromTargetClass(ObjectClass, InOutData);
}

bool UAudioPropertiesParserNameMatch::ParseProperties(TObjectPtr<UObject> TargetObject, const FAudioPropertiesSheet& PropertiesToParse) const
{
	UE::AudioGameplay::FAudioPropertiesParsingData ParsingData;
	
	if (!GenerateParsingData(TargetObject, PropertiesToParse, ParsingData))
	{
		return false;
	}

	FAudioPropertiesParser::InjectPropertyValuesIntoObject(*TargetObject, ParsingData);
	return true;	
}

#if WITH_EDITOR

void UAudioPropertiesParserNameMatch::FitPropertiesOnAssetTree(UAudioPropertiesSheetAsset& InSheetAsset) const
{
	if (!ValidationClass)
	{
		return;
	}

	UE::AudioGameplay::FAudioPropertiesParsingData ParsingData;
	
	if (!GenerateParsingDataFromTargetClass(ValidationClass, ParsingData))
	{
		return;
	}

	FAudioPropertiesParser::ApplyTargetMetadataToSheetTree(InSheetAsset, ParsingData);
}


bool UAudioPropertiesParserNameMatch::ValidatePropertiesOnAssetTree(const UAudioPropertiesSheetAsset& InSheetAsset) const
{
	if (!ValidationClass)
	{
		return true;
	}

	UE::AudioGameplay::FAudioPropertiesParsingData ParsingData;
	ParsingData.SourcePropertySheet = &InSheetAsset.PropertiesSheet;
	
	if (!GenerateParsingDataFromTargetClass(ValidationClass, ParsingData))
	{
		return false;
	}

	const bool bPropertiesNeedMetadata = FAudioPropertiesParser::ShouldApplyTargetMetadataToProperties(ParsingData);

	return !bPropertiesNeedMetadata;
}

bool UAudioPropertiesParserNameMatch::GenerateParsingDataFromTargetClass(UClass* TargetClass, UE::AudioGameplay::FAudioPropertiesParsingData& InOutData) const
{
	if (!TargetClass)
	{
		return false;
	}

	InOutData.TargetClass = TargetClass;

	FAudioPropertiesParser::GenerateNameMatchedParsingData(InOutData);
	return true;
}

#endif
