// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesSheet.h"

#include "AudioPropertiesBindings.h"
#include "AudioPropertiesLogs.h"
#include "AudioPropertiesParserBase.h"
#include "AudioPropertiesParsingData.h"
#include "AudioPropertiesUtils.h"
#include "Interfaces/ITargetPlatform.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/PropertyAccessUtil.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include "AudioPropertiesParserNameMatch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioPropertiesSheet)

#define LOCTEXT_NAMESPACE "AudioPropertiesSheetAsset"

namespace AudioPropertySheetPrivate
{
	bool SameType(const FPropertyBagPropertyDesc& Property1, const FPropertyBagPropertyDesc& Property2)
	{
		const bool bDifferentType = Property1.ValueType != Property2.ValueType || Property1.ValueTypeObject != Property2.ValueTypeObject;
		return !bDifferentType;
	}

	bool IsRelevantInstancedObjectProperty(const FProperty* TargetProperty, const UE::AudioGameplay::FAudioPropertiesParsingData& InstancedObjectParsingData)
	{
		check(TargetProperty)
		
		UClass* TargetObjectClass = InstancedObjectParsingData.TargetClass;

		if(!TargetObjectClass)
		{
			return false;
		}
		
		const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(TargetProperty);
		const FObjectProperty* ObjectProperty = ArrayProperty != nullptr ? CastField<FObjectProperty>(ArrayProperty->Inner) : CastField<FObjectProperty>(TargetProperty);

		if(!ObjectProperty)
		{
			return false;
		}

		const bool bIsInstanced = (ObjectProperty->PropertyFlags & CPF_InstancedReference) != 0;
		const bool bIsRelevantClass = InstancedObjectParsingData.bCreateNewObjects ? TargetObjectClass->IsChildOf(ObjectProperty->PropertyClass) : ObjectProperty->PropertyClass->IsChildOf(TargetObjectClass); 

		return bIsInstanced && bIsRelevantClass;
	}
}

bool FAudioPropertiesSheet::UpdatePropertyOverride(const FProperty& InProperty, bool bMarkAsOverridden)
{
	if (bMarkAsOverridden)
	{

		if (IsPropertyOverridden(InProperty))
		{
			return true;
		}

		AudioPropertiesSheet::PropertyDescParentPair PropertyDescParentPair = FindClosestParentWithProperty(InProperty);

		if (!PropertyDescParentPair.Key || !PropertyDescParentPair.Value)
		{
			return false;
 		}

		const FInstancedPropertyBag& ParentParams = PropertyDescParentPair.Value->PropertiesSheet.Properties;
		const FPropertyBagPropertyDesc* PropertyDescToInherit = PropertyDescParentPair.Key;

		//Make property local by cloning it into a new desc 
		const EPropertyBagContainerType ContainerType = PropertyDescToInherit->ContainerTypes.GetFirstContainerType();
		FPropertyBagPropertyDesc NewPropertyDesc(InProperty.GetFName(), ContainerType, PropertyDescToInherit->ValueType, PropertyDescToInherit->ValueTypeObject.Get());
		Properties.AddProperties({ NewPropertyDesc });
 		
 		//Copy value from parent property
		const FPropertyBagPropertyDesc* OverridenPropertyDesc = Properties.FindPropertyDescByName(InProperty.GetFName());
		const void* SourceAddr = PropertyDescToInherit->CachedProperty->ContainerPtrToValuePtr<void>(ParentParams.GetValue().GetMemory());
		void* DestinationAddr = OverridenPropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>(Properties.GetMutableValue().GetMemory());
		OverridenPropertyDesc->CachedProperty->CopyCompleteValue(DestinationAddr, SourceAddr);
		
		OnAudioPropertyOverrideChange.Broadcast(InProperty, bMarkAsOverridden);

		return true;
	}
	else
	{
		Properties.RemovePropertyByName(InProperty.GetFName());

		OnAudioPropertyOverrideChange.Broadcast(InProperty, bMarkAsOverridden);

		return true;
	}
 }

bool FAudioPropertiesSheet::IsPropertyOverridden(const FProperty& InProperty) const
{
	const FPropertyBagPropertyDesc* LocalPropertyDesc = Properties.FindPropertyDescByName(InProperty.GetFName());
	const FPropertyBagPropertyDesc* ParentPropertyDesc = FindClosestParentWithProperty(InProperty).Key;
	
	if (LocalPropertyDesc && ParentPropertyDesc)
	{
		return AudioPropertySheetPrivate::SameType(*LocalPropertyDesc, *ParentPropertyDesc);
	}

	return false;
}

AudioPropertiesSheet::PropertyDescParentPair FAudioPropertiesSheet::FindClosestParentWithProperty(const FProperty& InProperty) const
{
	if (!Parent)
	{
		return AudioPropertiesSheet::PropertyDescParentPair(nullptr, nullptr);
	}

	TObjectPtr<const UAudioPropertiesSheetAsset> VisitedParent = Parent;

	while (VisitedParent)
	{
		if (const FPropertyBagPropertyDesc* ParentDesc = VisitedParent->PropertiesSheet.Properties.FindPropertyDescByName(InProperty.GetFName()))
		{
			return AudioPropertiesSheet::PropertyDescParentPair(ParentDesc, VisitedParent);
		}

		VisitedParent = VisitedParent->PropertiesSheet.Parent;
	}

	return AudioPropertiesSheet::PropertyDescParentPair(nullptr, nullptr);;
}


void FAudioPropertiesSheet::ReconcileProperties()
{
	if (!Properties.GetPropertyBagStruct())
	{
		return;
	}

	TSet<FGuid> MatchingOverrides;
	TArray<FPropertyBagPropertyDesc> NewDescs;
	TArray<TPair<const FPropertyBagPropertyDesc&, const FInstancedPropertyBag&>> PropertiesToCopyValueFrom;
	
	//bag to store old properties values
	FInstancedPropertyBag OldProperties;

	for (const FPropertyBagPropertyDesc& LocalPropertyDesc : Properties.GetPropertyBagStruct()->GetPropertyDescs())
	{
		check(LocalPropertyDesc.CachedProperty)
		
		// Check if a parent sheet contains the property
		AudioPropertiesSheet::PropertyDescParentPair PropertyDescParentPair = FindClosestParentWithProperty(*LocalPropertyDesc.CachedProperty);

		if (PropertyDescParentPair.Value)
		{
			const FPropertyBagPropertyDesc* ParentPropertyDesc = PropertyDescParentPair.Key;

			if (ParentPropertyDesc)
			{
				if (!AudioPropertySheetPrivate::SameType(LocalPropertyDesc, *ParentPropertyDesc))
				{
					UE_LOGF(LogAudioProperties, Warning, "Parent type changed for property %ls, property type will be changed and value might be lost", *LocalPropertyDesc.CachedProperty->GetFName().ToString());
					NewDescs.Emplace(LocalPropertyDesc.CachedProperty->GetFName(), ParentPropertyDesc->ContainerTypes.GetFirstContainerType(), ParentPropertyDesc->ValueType, ParentPropertyDesc->ValueTypeObject.Get());
					
					if (PropertyAccessUtil::ArePropertiesCompatible(LocalPropertyDesc.CachedProperty, ParentPropertyDesc->CachedProperty))
					{
						//if the local property is compatible with the new type we keep its value
						PropertiesToCopyValueFrom.Emplace(LocalPropertyDesc, OldProperties);
					}
					else
					{
						//otherwise we initialize using the parent value
						PropertiesToCopyValueFrom.Emplace(*ParentPropertyDesc, PropertyDescParentPair.Value->PropertiesSheet.Properties);
					}
					continue;

				}
			}
		}

		NewDescs.Add(LocalPropertyDesc);
	}
	
	OldProperties.MigrateToNewBagInstance(Properties);

	//update params from reconciled descriptions
	const UPropertyBag* NewBag = UPropertyBag::GetOrCreateFromDescs(NewDescs);
	Properties.MigrateToNewBagStruct(NewBag);

	//now that we instantiated the underlying UObject holding FProperties, copy old values
	for (TPair<const FPropertyBagPropertyDesc&, const FInstancedPropertyBag&> PropertyToCopy : PropertiesToCopyValueFrom)
	{
		const void* SourceAddr = PropertyToCopy.Key.CachedProperty->ContainerPtrToValuePtr<void>(PropertyToCopy.Value.GetValue().GetMemory());

		const FPropertyBagPropertyDesc* TargetPropertyDesc = NewBag->FindPropertyDescByName(PropertyToCopy.Key.CachedProperty->GetFName());
		void* TargetAddr = TargetPropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>(Properties.GetMutableValue().GetMemory());
		
		PropertyAccessUtil::CopyCompletePropertyValue(PropertyToCopy.Key.CachedProperty, SourceAddr, TargetPropertyDesc->CachedProperty, TargetAddr);
	}
}

void FAudioPropertiesSheet::OnPreSave(const FObjectPreSaveContext& SaveContext)
{
	ReconcileProperties();

	if(!SaveContext.IsCooking())
	{
		return;
	}

	if (!SaveContext.GetTargetPlatform()->HasEditorOnlyData())
	{
		return;
	}

	FlattenSheet();
}

void FAudioPropertiesSheet::FlattenSheet()
{
	if (!Properties.GetPropertyBagStruct())
	{
		return;
	}

	ReconcileProperties();

	auto ParseLeafMostNonLocalProperties = [this, &LocalPropertyBag = Properties](const FPropertyBagPropertyDesc& PropertyDesc, const FAudioPropertiesSheet& OwningSheet)
	{
		if (this == &OwningSheet)
		{
			return;
		}

		if (!PropertyDesc.CachedProperty)
		{
			return;
		}

		LocalPropertyBag.AddProperty(PropertyDesc.CachedProperty->GetFName(), PropertyDesc.CachedProperty);

		const void* SourcePropertyAddr = PropertyDesc.CachedProperty->ContainerPtrToValuePtr<void>(OwningSheet.Properties.GetValue().GetMemory());
		LocalPropertyBag.SetValue(PropertyDesc.CachedProperty->GetFName(), PropertyDesc.CachedProperty, SourcePropertyAddr);

	};

	AudioPropertiesUtils::VisitLeafMostProperties(*this, ParseLeafMostNonLocalProperties);

	Parent = nullptr;
}

UAudioPropertiesSheetAsset::UAudioPropertiesSheetAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	//if we don't have a parser (or are inheriting one) create a default one
	if (!PropertiesParser && !PropertiesSheet.Parent)
	{
		PropertiesParser = ObjectInitializer.CreateDefaultSubobject<UAudioPropertiesParserNameMatch>(this, TEXT("DefaultPropertiesParser"));
	}
}

#if WITH_EDITOR

void UAudioPropertiesSheetAsset::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
	
	PropertiesSheet.OnPreSave(SaveContext);
}

EDataValidationResult UAudioPropertiesSheetAsset::IsDataValid(FDataValidationContext& Context) const
{
	Super::IsDataValid(Context);

	FString AssetName;
	GetName(AssetName);

	UE_LOGF(LogAudioProperties, Log, "Validating Property Sheet %ls", *AssetName);

	if (PropertiesParser && !PropertiesParser->ValidatePropertiesOnAssetTree(*this))
	{
		Context.AddWarning(LOCTEXT("Parser Validation failed", "Parser validation failed, check output log"));

		UE_LOGF(LogAudioProperties, Log, "Validating Property Sheet %ls - Validation failed", *AssetName);

		return EDataValidationResult::Invalid;
	}

	UE_LOGF(LogAudioProperties, Log, "Validating Property Sheet %ls - Validation successfull", *AssetName);

	return EDataValidationResult::Valid;
}

void UAudioPropertiesSheetAsset::FitPropertiesForValidation()
{
	TObjectPtr<const UAudioPropertiesParserBase> ParserToUse = FindClosestParserInInheritanceTree().Key;

	if (!ParserToUse)
	{
		UE_LOGF(LogAudioProperties, Log, "Trying to validate data with null parser")
		return;
	}
	
	ParserToUse->FitPropertiesOnAssetTree(*this);

}

bool UAudioPropertiesSheetAsset::CopyToObjectProperties(UObject* TargetObject) const
{
	TObjectPtr<const UAudioPropertiesParserBase> ParserToUse = FindClosestParserInInheritanceTree().Key;

	if (!ParserToUse)
	{

		FString AssetName;
		GetName(AssetName);

		UE_LOGF(LogAudioProperties, Log, "Failed to inject data from property sheet %ls - could not find a valid parser", *AssetName)
		return false;
	}

	return ParserToUse->ParseProperties(TargetObject, PropertiesSheet);

}

FDelegateHandle UAudioPropertiesSheetAsset::BindPropertiesCopyToSheetChanges(UObject* TargetObject)
{
	auto OnPropertySheetChanged = [this, TargetObject](AudioPropertiesSheet::PostEditChangeType ChangeType) 
		{ 
			this->CopyToObjectProperties(TargetObject); 
		};
	
	return OnPostEditChange.AddWeakLambda(TargetObject, OnPropertySheetChanged);
}

void UAudioPropertiesSheetAsset::UnbindCopyFromPropertySheetChanges(UObject* ObjectToUnbind)
{
	OnPostEditChange.RemoveAll(ObjectToUnbind);
}

void UAudioPropertiesSheetAsset::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (!PropertyAboutToChange)
	{
		return;
	}

	if (PropertyAboutToChange->GetName() == GET_MEMBER_NAME_CHECKED(FAudioPropertiesSheet, Parent))
	{
		TObjectPtr<const UAudioPropertiesSheetAsset> ParentValue = nullptr;
		PropertyAboutToChange->GetValue_InContainer(&PropertiesSheet, &ParentValue);

		if (ParentValue)
		{
			UAudioPropertiesSheetAsset* ParentPtr = const_cast<UAudioPropertiesSheetAsset*>(ParentValue.Get());
			ParentPtr->OnPostEditChange.RemoveAll(this);
		}
	}
}

void UAudioPropertiesSheetAsset::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	AudioPropertiesSheet::PostEditChangeType BroadcastChangeType = AudioPropertiesSheet::PostEditChangeType::Other;

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FAudioPropertiesSheet, Parent))
	{
		TObjectPtr<const UAudioPropertiesSheetAsset> ParentValue = nullptr;
		TObjectPtr<const UAudioPropertiesSheetAsset> AssetBeingChecked = this;
		PropertyChangedEvent.Property->GetValue_InContainer(&PropertiesSheet, &ParentValue);

		if (ParentValue)
		{
			UAudioPropertiesSheetAsset* ParentPtr = const_cast<UAudioPropertiesSheetAsset*>(ParentValue.Get());
			ParentPtr->OnPostEditChange.AddWeakLambda(this, [this](AudioPropertiesSheet::PostEditChangeType ChangeType) {this->OnParentPostEditChange(ChangeType); });
		}

		while (ParentValue)
		{
			checkf(ParentValue != this, TEXT("Found asset %s while walking its parent chain at %s"), *this->GetName(), *AssetBeingChecked->GetName());
			AssetBeingChecked = ParentValue;
			ParentValue = ParentValue->PropertiesSheet.Parent;
		}

		PropertiesSheet.ReconcileProperties();
		BroadcastChangeType = AudioPropertiesSheet::PostEditChangeType::ParentChange;
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FAudioPropertiesSheet, Properties))
	{
		PropertiesSheet.ReconcileProperties();
		BroadcastChangeType = AudioPropertiesSheet::PostEditChangeType::PropertyBag;

	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UAudioPropertiesSheetAsset, PropertiesParser))
	{
		BroadcastChangeType = AudioPropertiesSheet::PostEditChangeType::PropertiesParser;
	}

	OnPostEditChange.Broadcast(BroadcastChangeType);
}

void UAudioPropertiesSheetAsset::PostLoad()
{
	Super::PostLoad();

	PropertiesSheet.ReconcileProperties();

	if (PropertiesSheet.Parent)
	{
		UAudioPropertiesSheetAsset* ParentPtr = const_cast<UAudioPropertiesSheetAsset*>(PropertiesSheet.Parent.Get());
		ParentPtr->OnPostEditChange.AddWeakLambda(this, [this](AudioPropertiesSheet::PostEditChangeType ChangeType) 
			{
				this->OnParentPostEditChange(ChangeType); 
			}
		);
	}
}

#endif

void UAudioPropertiesSheetAsset::OnParentPostEditChange(const AudioPropertiesSheet::PostEditChangeType& ChangeType)
{
	switch (ChangeType)
	{
	case AudioPropertiesSheet::PostEditChangeType::PropertyBag:
	case AudioPropertiesSheet::PostEditChangeType::ParentChange:
		PropertiesSheet.ReconcileProperties();
		MarkPackageDirty();
		break;
	case AudioPropertiesSheet::PostEditChangeType::Other:
		break;
	default:
		break;
	}

	OnPostEditChange.Broadcast(ChangeType);
}

AudioPropertiesSheet::ParserParentPair UAudioPropertiesSheetAsset::FindClosestParserInInheritanceTree() const
{
	if (PropertiesParser)
	{
		return AudioPropertiesSheet::ParserParentPair(PropertiesParser, this);
	}

	TObjectPtr<const UAudioPropertiesParserBase> ClosestParser = nullptr;
	const UAudioPropertiesSheetAsset* ParentToVisit = PropertiesSheet.Parent; 
	const UAudioPropertiesSheetAsset* OutParent = PropertiesSheet.Parent; 


	while (ParentToVisit && ClosestParser == nullptr)
	{
		OutParent = ParentToVisit;
		ClosestParser = ParentToVisit->PropertiesParser;
		ParentToVisit = ParentToVisit->PropertiesSheet.Parent;
	}

	return AudioPropertiesSheet::ParserParentPair(ClosestParser, ClosestParser ? OutParent : nullptr);
}

void UAudioPropertiesSheetAsset::GetTargetedPropertyNames(TObjectPtr<const UObject> TargetObject, TArray<FName>& OutProperties) const
{
	TObjectPtr<const UAudioPropertiesParserBase> Parser = FindClosestParserInInheritanceTree().Key;

	if (!Parser)
	{
		return;
	}

	UE::AudioGameplay::FAudioPropertiesParsingData ParsingData;
	Parser->GenerateParsingData(TargetObject, PropertiesSheet, ParsingData);

	auto OnLeafPropertyVisited = [OutPropertyNames = &OutProperties, Bindings = &ParsingData.SourceToTargetPropertyBindings](const FPropertyBagPropertyDesc& PropertyDesc, const FAudioPropertiesSheet& OwningSheet) 
	{
		if (const FName* TargetPropertyName = Bindings->Find(PropertyDesc.Name))
		{
			OutPropertyNames->Add(*TargetPropertyName);
		}
	};

	AudioPropertiesUtils::VisitLeafMostProperties(PropertiesSheet, OnLeafPropertyVisited);

	for(const UE::AudioGameplay::FAudioPropertiesParsingData& InstancedObjectParsingData : ParsingData.InstancedObjectsParsingData)
	{
		if(InstancedObjectParsingData.SourceToTargetPropertyBindings.Num() > 0 || InstancedObjectParsingData.bCreateNewObjects)
		{
			UClass* InstancedObjectClass = InstancedObjectParsingData.TargetClass;

			if(!InstancedObjectClass)
			{
				continue;
			}

			for (TFieldIterator<FProperty> PropIt(TargetObject->GetClass()); PropIt; ++PropIt)
			{
				const FProperty* Property = *PropIt;

				if(!ensureAlwaysMsgf(Property, TEXT("Fsound a null property from a field iterator")))
				{
					continue;
				}
				
				if (AudioPropertySheetPrivate::IsRelevantInstancedObjectProperty(Property, InstancedObjectParsingData))
				{
					OutProperties.Add(Property->GetFName());
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
