// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesUtils.h"

#include "AudioPropertiesSheet.h"

#include "AssetRegistry/AssetRegistryModule.h"

bool AudioPropertiesUtils::IsSheetInInheritanceTree(const UAudioPropertiesSheetAsset& InLeafSheet, const TObjectPtr<const UAudioPropertiesSheetAsset> InSheetToFind)
{
	check(InSheetToFind)

	TObjectPtr<const UAudioPropertiesSheetAsset> ParentValue = InLeafSheet.PropertiesSheet.Parent;

	while (ParentValue)
	{
		if (ParentValue == InSheetToFind)
		{
			return true;
		}

		ParentValue = ParentValue->PropertiesSheet.Parent;
	}

	return false;
}

void AudioPropertiesUtils::GetAudioPropertiesSheetAssetReferencers(const UAudioPropertiesSheetAsset& InAsset, TArray<FAssetData>& OutReferencersData, const UClass* ClassFilter /*= nullptr*/)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	TArray<FAssetIdentifier> AssetDependencies;
	FARFilter AssetRegistryFilter;

	if (ClassFilter)
	{
		FTopLevelAssetPath ClassPathName = ClassFilter->GetClassPathName();
		AssetRegistryFilter.ClassPaths.Add(ClassPathName);

	}

	AssetRegistry.GetReferencers(InAsset.GetPackage()->GetFName(), AssetRegistryFilter.PackageNames);
	AssetRegistry.GetAssets(AssetRegistryFilter, OutReferencersData);
}

void AudioPropertiesUtils::VisitAudioPropertiesSheetAssetReferencers(const UAudioPropertiesSheetAsset& InAsset, TFunctionRef<void(FAssetData&)> OnAssetVisited, const UClass* ClassFilter /*= nullptr*/)
{
	TArray<FAssetData> ReferencersData;
	GetAudioPropertiesSheetAssetReferencers(InAsset, ReferencersData, ClassFilter);

	for (FAssetData& AssetData : ReferencersData)
	{
		OnAssetVisited(AssetData);
	}
}

void AudioPropertiesUtils::VisitLeafMostProperties(const UAudioPropertiesSheetAsset& InSheetAsset, TFunction<void(const FPropertyBagPropertyDesc&, const FAudioPropertiesSheet&)> OnPropertyVisited)
{
	VisitLeafMostProperties(InSheetAsset.PropertiesSheet, OnPropertyVisited);
}

void AudioPropertiesUtils::VisitLeafMostProperties(const FAudioPropertiesSheet& InSheetAsset, TFunction<void(const FPropertyBagPropertyDesc&, const FAudioPropertiesSheet&)> OnPropertyVisited)
{
	const FAudioPropertiesSheet* LeafSheet = &InSheetAsset;
	const FAudioPropertiesSheet* SourceSheet = &InSheetAsset;

	while (SourceSheet != nullptr)
	{
		const FInstancedPropertyBag& SheetProperties = SourceSheet->Properties;

		if (const UPropertyBag* BagStruct = SheetProperties.GetPropertyBagStruct())
		{
			for (const FPropertyBagPropertyDesc& PropertyDesc : BagStruct->GetPropertyDescs())
			{
				const bool bSkipOverridenProperty = SourceSheet != &InSheetAsset && LeafSheet->IsPropertyOverridden(*PropertyDesc.CachedProperty);

				if (!bSkipOverridenProperty)
				{
					OnPropertyVisited(PropertyDesc, *SourceSheet);
				}

			}
		}

		SourceSheet = SourceSheet->Parent ? &SourceSheet->Parent->PropertiesSheet : nullptr;;
	}
}
