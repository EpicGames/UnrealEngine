// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "AudioPropertiesBindings.generated.h"

UCLASS(MinimalAPI)
class UAudioPropertiesBindings : public UObject
{
	GENERATED_BODY()

	UAudioPropertiesBindings() = default;

public:
	UPROPERTY(EditDefaultsOnly, Category = "Default")
	TMap<FName, FName> ObjectPropertyToSheetPropertyMap;
};
