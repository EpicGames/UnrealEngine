// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioPropertiesSheet.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"

#include "AudioPropertiesParserBase.generated.h"

struct FAudioPropertiesSheet;

namespace UE
{
	namespace AudioGameplay
	{
		struct FAudioPropertiesParsingData;
	}
}

USTRUCT()
struct FInstancedObjectParsingInstructions
{
	GENERATED_BODY()

	//The instanced object class that will be targeted by these instructions
	UPROPERTY(EditAnywhere, Category = "Parsing Instructions")
	TSubclassOf<UObject> InstancedObjectClass;

	//If the target instanced uobject is missing, create one with the properties that should have been parsed
	UPROPERTY(EditAnywhere, Category = "Parsing Instructions")
	bool bCreateIfMissing = false;

	//Use this to signal that you would like to use a different parser for the instanced objects
	UPROPERTY(EditAnywhere, Category = "Parsing Instructions")
	bool bOverrideParsing = false;
	
	UPROPERTY(EditAnywhere, Instanced, Category = "Parsing Instructions", meta = (EditCondition = "bOverrideParsing", EditConditionHides))
	TObjectPtr<UAudioPropertiesParserBase> PropertiesParser;
};



UCLASS(Abstract, EditInlineNew, MinimalAPI)
class UAudioPropertiesParserBase : public UObject
{
	GENERATED_BODY()

public:
	virtual bool GenerateParsingData(TObjectPtr<const UObject> TargetObject,  const FAudioPropertiesSheet& SourceSheet, UE::AudioGameplay::FAudioPropertiesParsingData& InOutData) const { return false; };
	virtual bool ParseProperties(TObjectPtr<UObject> TargetObject, const FAudioPropertiesSheet& PropertiesToParse) const { return false; };

#if WITH_EDITOR
	virtual bool ValidatePropertiesOnAssetTree(const UAudioPropertiesSheetAsset& InSheetAsset) const { return false; }
	virtual void FitPropertiesOnAssetTree(UAudioPropertiesSheetAsset& InSheetAsset) const { return; }
#endif

};