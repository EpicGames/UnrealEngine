// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioPropertiesParserNameMatch.h"

#include "AudioPropertiesParserBindings.generated.h"

class UAudioPropertiesBindings;
struct FAudioPropertiesSheet;

/*
*	UAudioPropertiesParserBindings
* 
*	Parses values from a Property Sheet to a target UObject 
*	following the name mappings defined in the BindingsAsset.
* 
*	Properties need to be of compatible types for values to be parsed.
* 
*/
UCLASS(MinimalAPI)
class UAudioPropertiesParserBindings : public UAudioPropertiesParserNameMatch
{
	GENERATED_BODY()

public:
	//The bindings that will be used to parse properties from the Sheet to the target asset
	UPROPERTY(EditAnywhere, Category = "AudioProperties")
	TObjectPtr<UAudioPropertiesBindings> BindingsAsset;

	virtual bool GenerateParsingData(TObjectPtr<const UObject> TargetObject, const FAudioPropertiesSheet& SourceSheet, UE::AudioGameplay::FAudioPropertiesParsingData& InOutData) const override;
	virtual bool ParseProperties(TObjectPtr<UObject> TargetObject, const FAudioPropertiesSheet& PropertiesToParse) const override;

protected:

#if WITH_EDITOR
	virtual bool GenerateParsingDataFromTargetClass(UClass* TargetClass, UE::AudioGameplay::FAudioPropertiesParsingData& InOutData) const override;
#endif
};