// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioPropertiesParserBase.h"
#include "Templates/SubclassOf.h"

#include "AudioPropertiesParserNameMatch.generated.h"

/*
*	UAudioPropertiesParserNameMatch
* 
*	Parses values from a Property Sheet to a target UObject 
*	of compatible type properties with the same name
* 
*/
UCLASS(MinimalAPI)
class UAudioPropertiesParserNameMatch : public UAudioPropertiesParserBase
{
	GENERATED_BODY()

public:
	virtual bool GenerateParsingData(TObjectPtr<const UObject> TargetObject, const FAudioPropertiesSheet& SourceSheet, UE::AudioGameplay::FAudioPropertiesParsingData& InOutData) const override;
	virtual bool ParseProperties(TObjectPtr<UObject> TargetObject, const FAudioPropertiesSheet& PropertiesToParse) const override;

	UPROPERTY(EditAnywhere, Category = "Instanced Objects Parsing")
	TArray<FInstancedObjectParsingInstructions> InstancedObjectsRules;
	
#if WITH_EDITOR
	bool ValidatePropertiesOnAssetTree(const UAudioPropertiesSheetAsset& InSheetAsset) const override;
	virtual void FitPropertiesOnAssetTree(UAudioPropertiesSheetAsset& InSheetAsset) const override;

	UPROPERTY(EditAnywhere, Category = "Validation")
	TSubclassOf<UObject> ValidationClass;
#endif

protected:

#if WITH_EDITOR
	virtual bool GenerateParsingDataFromTargetClass(UClass* TargetClass, UE::AudioGameplay::FAudioPropertiesParsingData& InOutData) const;
#endif
};