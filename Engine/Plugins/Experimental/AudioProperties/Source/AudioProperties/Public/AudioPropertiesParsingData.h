// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "UObject/NameTypes.h"

class UClass;
struct FAudioPropertiesSheet;

namespace UE
{
	namespace AudioGameplay
	{
		using FAudioPropertiesBindings = TMap<FName, FName>;

		struct FAudioPropertiesParsingData
		{
			const FAudioPropertiesSheet* SourcePropertySheet = nullptr;
			UClass* TargetClass = nullptr;
			FAudioPropertiesBindings SourceToTargetPropertyBindings;
			bool bCreateNewObjects = false;
			TArray<FAudioPropertiesParsingData> InstancedObjectsParsingData;
		};
	}
}

