// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioPropertiesSheetAssetBase.h"
#include "Delegates/Delegate.h"
#include "StructUtils/PropertyBag.h"

#include "AudioPropertiesSheet.generated.h"

#define UE_API AUDIOPROPERTIES_API

class IAudioPropertiesParser;
class UAudioPropertiesBindings;
class UAudioPropertiesParserBase;

namespace AudioPropertiesSheet
{
	typedef TPair<const FPropertyBagPropertyDesc*, TObjectPtr<const UAudioPropertiesSheetAsset>> PropertyDescParentPair;
	typedef TPair<TObjectPtr<const UAudioPropertiesParserBase>, TObjectPtr<const UAudioPropertiesSheetAsset>> ParserParentPair;

	#if WITH_EDITOR
	enum class PostEditChangeType : uint8
	{
		Other,
		ParentChange, 
		PropertyBag,
		PropertiesParser
	};
#endif
}

#if WITH_EDITOR
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAudioPropertySheetPostEditChange, AudioPropertiesSheet::PostEditChangeType);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAudioPropertyOverride, const FProperty& /*TargetProperty*/, bool /*bIsOverridden*/);
#endif

USTRUCT()
struct FAudioPropertiesSheet
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Default")
	TObjectPtr<const UAudioPropertiesSheetAsset> Parent;

	UPROPERTY(EditAnywhere, Category = "Default")
	FInstancedPropertyBag Properties;

	UE_API bool UpdatePropertyOverride(const FProperty& InProperty, bool bMarkAsOverridden);

	UE_API bool IsPropertyOverridden(const FProperty& InProperty) const;

	UE_API void ReconcileProperties();

	UE_API void OnPreSave(const FObjectPreSaveContext& SaveContext);

	UE_API AudioPropertiesSheet::PropertyDescParentPair FindClosestParentWithProperty(const FProperty& InProperty) const;

#if WITH_EDITOR
	FOnAudioPropertyOverride OnAudioPropertyOverrideChange;
#endif	
private:
	UE_API void FlattenSheet();

};

UCLASS(MinimalAPI, BlueprintType)
class UAudioPropertiesSheetAsset : public UAudioPropertiesSheetAssetBase
{
	GENERATED_BODY()

public:
	UE_API UAudioPropertiesSheetAsset(const FObjectInitializer& ObjectInitializer);

	UPROPERTY(EditAnywhere, Category = "Default")
	FAudioPropertiesSheet PropertiesSheet;

		//The properties parser determines how the properties on this sheet are applied to target UObjects
	//NOTE: A null parser will mean no properties are applied - a child sheet will however inherit the parent parser automatically
	UPROPERTY(EditAnywhere, Instanced, Category = "Parser")
	TObjectPtr<UAudioPropertiesParserBase> PropertiesParser;

#if WITH_EDITOR
	//Calling this method will apply the parser validation rules to the sheet
	//NOTE will dirty this asset and all the parents where property changes are applied
	UFUNCTION(CallInEditor, meta=(DisplayName="Fit Properties For Validation"))
	UE_API void FitPropertiesForValidation();

	UE_API virtual bool CopyToObjectProperties(UObject* TargetObject) const override;

	UE_API virtual FDelegateHandle BindPropertiesCopyToSheetChanges(UObject* TargetObject) override;
	UE_API virtual void UnbindCopyFromPropertySheetChanges(UObject* ObjectToUnbind) override;

	UE_API void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	FOnAudioPropertySheetPostEditChange OnPostEditChange;

	UE_API void PostLoad();

	UE_API EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

	UE_API AudioPropertiesSheet::ParserParentPair FindClosestParserInInheritanceTree() const;

	UE_API void GetTargetedPropertyNames(TObjectPtr<const UObject> TargetObject, TArray<FName>& OutProperties) const;

	UE_API void OnParentPostEditChange(const AudioPropertiesSheet::PostEditChangeType& ChangeType);
#endif 

	UE_API void PreSave(FObjectPreSaveContext SaveContext) override;

};

#undef UE_API
