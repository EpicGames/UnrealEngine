// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "AssetRegistry/AssetData.h"

class UAudioPropertiesSheetAsset;
struct FPropertyBagPropertyDesc;
struct FAudioPropertiesSheet;

namespace AudioPropertiesUtils
{
	static const FLazyName AudioPropertiesOverrideTagName = "AudioPropertyOverride";

	bool AUDIOPROPERTIES_API IsSheetInInheritanceTree(const UAudioPropertiesSheetAsset& InLeafSheet, const TObjectPtr<const UAudioPropertiesSheetAsset> InSheetToFind);
	void GetAudioPropertiesSheetAssetReferencers(const UAudioPropertiesSheetAsset& InAsset, TArray<FAssetData>& OutReferencersData, const UClass* ClassFilter = nullptr);
	void VisitAudioPropertiesSheetAssetReferencers(const UAudioPropertiesSheetAsset& InAsset, TFunctionRef<void(FAssetData&)> OnAssetVisited, const UClass* ClassFilter = nullptr);
	void AUDIOPROPERTIES_API VisitLeafMostProperties(const UAudioPropertiesSheetAsset& InSheetAsset, TFunction<void(const FPropertyBagPropertyDesc& /*Property Desc */, const FAudioPropertiesSheet& /*Owning Sheet*/)> OnPropertyVisited);
	void VisitLeafMostProperties(const FAudioPropertiesSheet& InSheetAsset, TFunction<void(const FPropertyBagPropertyDesc& /*Property Desc */, const FAudioPropertiesSheet& /*Owning Sheet*/)> OnPropertyVisited);


}