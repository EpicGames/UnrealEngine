// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AudioPropertiesEditor : ModuleRules
	{
		public AudioPropertiesEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] 
				{
					"AudioExtensions",
					"AudioProperties",
					"AssetDefinition",
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"PropertyEditor",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"StructUtilsEditor",
					"UnrealEd"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] 
				{
					"AssetDefinition",
					"AudioEditor"
				}
			);
		}
	}
}