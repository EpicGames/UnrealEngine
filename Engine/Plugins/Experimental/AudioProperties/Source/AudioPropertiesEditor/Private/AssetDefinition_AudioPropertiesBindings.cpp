// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AudioPropertiesBindings.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_AudioPropertiesBindings"

TConstArrayView<FAssetCategoryPath> UAssetDefinition_AudioPropertiesBindings::GetAssetCategories() const
{
	static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio, 
			LOCTEXT("AssetAudioPropertiesBindingSubMenu", "Advanced"),
				FCategoryPath(LOCTEXT("AssetAudioPropertiesBindingSubMenuSection", "Properties"), ECategoryMenuType::Section))
		};
	return Categories;
}

#undef LOCTEXT_NAMESPACE
