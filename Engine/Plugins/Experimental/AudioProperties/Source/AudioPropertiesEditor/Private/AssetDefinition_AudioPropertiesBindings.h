// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "AudioPropertiesBindings.h"
#include "AudioPropertiesEditorModule.h"

#include "AssetDefinition_AudioPropertiesBindings.generated.h"

UCLASS()
class UAssetDefinition_AudioPropertiesBindings : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	//~ Begin UAssetDefinition
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AudioPropertiesBindings", "Audio Properties Bindings"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(AudioPropertiesEditorModule::AssetColor); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAudioPropertiesBindings::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	//~ End UAssetDefinition
};
