// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AudioPropertiesSheetAsset.h"

#include "ContentBrowserMenuContexts.h"
#include "Toolkits/SimpleAssetEditor.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AudioPropertiesEditor"

TConstArrayView<FAssetCategoryPath> UAssetDefinition_AudioPropertiesSheetAsset::GetAssetCategories() const
{
	static const auto Categories = 
	{
		FAssetCategoryPath(EAssetCategoryPaths::Audio, 
			LOCTEXT("AssetAudioPropertiesSheetSubMenu", "Advanced"),
			FCategoryPath(LOCTEXT("AssetAudioPropertiesSheetSubMenuSection", "Properties"), ECategoryMenuType::Section))
	};
	return Categories;
}

EAssetCommandResult UAssetDefinition_AudioPropertiesSheetAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UAudioPropertiesSheetAsset* PropertySheetAsset : OpenArgs.LoadObjects<UAudioPropertiesSheetAsset>())
	{
		FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, PropertySheetAsset);
	}
	
	return EAssetCommandResult::Handled;
}

FToolMenuSection* UAssetDefinition_AudioPropertiesSheetAsset::FindContextMenuSection(FName SectionName) const
{
	TArray<FToolMenuSection*> Sections = RebuildContextMenuSections();
	
	for (FToolMenuSection* Section : Sections)
	{
		if (Section->Name == SectionName)
		{
			return Section;
		}
	}

	return nullptr;
}

TArray<FToolMenuSection*> UAssetDefinition_AudioPropertiesSheetAsset::RebuildContextMenuSections() const
{
	TArray<FToolMenuSection*> Sections;
	
	UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(GetAssetClass());
	check(Menu);

	FToolMenuSection& SoundSection = Menu->FindOrAddSection("Builder");
	SoundSection.Label = LOCTEXT("AudioPropertiesSheetAssetActions_Label", "Builder");

	Sections.Add(&SoundSection);

	return Sections;
}

#undef LOCTEXT_NAMESPACE
