// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesBindingsFactory.h"

#include "AudioPropertiesBindings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioPropertiesBindingsFactory)

UAudioPropertiesBindingsFactory::UAudioPropertiesBindingsFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAudioPropertiesBindings::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}


UObject* UAudioPropertiesBindingsFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAudioPropertiesBindings>(InParent, Name, Flags);
}
