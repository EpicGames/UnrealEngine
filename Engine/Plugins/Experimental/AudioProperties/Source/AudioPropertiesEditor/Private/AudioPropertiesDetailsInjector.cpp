// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesDetailsInjector.h"

#include "AudioPropertiesDetailsInjectorUtils.h"
#include "AudioPropertiesSheet.h"
#include "CustomNodeBuilders/AudioPropertiesDetailArrayBuilder.h"
#include "CustomNodeBuilders/AudioPropertiesDetailNestedNodesBuilder.h"
#include "CustomNodeBuilders/AudioPropertiesDetailSetBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "PropertyCustomizationHelpers.h"
#include "UObject/Object.h"

#define LOCTEXT_NAMESPACE "AudioPropertiesDetailsInjection"

void FAudioPropertiesDetailsInjector::CustomizeInjectedPropertiesDetails(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> PropertySheetPropertyHandle)
{
	if (!PropertySheetPropertyHandle->IsValidHandle())
	{
		return;
	}

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	CreateOverriddenPropertiesWidgets(DetailBuilder, ObjectsBeingCustomized, PropertySheetPropertyHandle);
}

void FAudioPropertiesDetailsInjector::CreateOverriddenPropertiesWidgets(IDetailLayoutBuilder& DetailBuilder, TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized, TSharedRef<IPropertyHandle> PropertySheetPropertyHandle)
{
	for (TWeakObjectPtr<UObject>& WeakObjectBeingCustomized : ObjectsBeingCustomized)
	{
		TObjectPtr<UObject> ObjectBeingCustomized = WeakObjectBeingCustomized.Get();

		if (!ObjectBeingCustomized || !ObjectBeingCustomized->IsValidLowLevel())
		{
			continue;
		}

		const UAudioPropertiesSheetAsset* SheetAssetPtr = Cast<UAudioPropertiesSheetAsset>(AudioPropertiesDetailsInjectorUtils::GetUObjectFromPtrPropertyHandle(PropertySheetPropertyHandle, ObjectBeingCustomized));

		if (!SheetAssetPtr || !SheetAssetPtr->IsValidLowLevel())
		{
			continue;
		}

		TArray<FName> TargetedPropertyNames;

		SheetAssetPtr->GetTargetedPropertyNames(ObjectBeingCustomized, TargetedPropertyNames);


		for (const FName& TargetPropertyName : TargetedPropertyNames)
		{
			TSharedRef<IPropertyHandle> OverriddenPropertyHandle = DetailBuilder.GetProperty(TargetPropertyName);

			if (!OverriddenPropertyHandle->IsValidHandle())
			{
				//If GetProperty fails use the class path and walk the inheritance tree 
				UClass* ObjectClass = ObjectBeingCustomized->GetClass();
				for (UClass* Class = ObjectClass; Class != nullptr; Class = Class->GetSuperClass())
				{
					OverriddenPropertyHandle = DetailBuilder.GetProperty(TargetPropertyName, Class);
					if (OverriddenPropertyHandle->IsValidHandle())
					{
						break;
					}
				}

				if (!OverriddenPropertyHandle->IsValidHandle())
				{
					continue;
				}
			}
			
			IDetailPropertyRow* OverriddenPropertyRow = DetailBuilder.EditDefaultProperty(OverriddenPropertyHandle);
			check(OverriddenPropertyRow)

			if (OverriddenPropertyHandle->AsArray().IsValid())
			{
				const FName DetailCategory = OverriddenPropertyHandle->GetDefaultCategoryName();
				TSharedRef<FAudioPropertiesDetailArrayBuilder> OverriddenArrayDetails = MakeShared<FAudioPropertiesDetailArrayBuilder>(OverriddenPropertyHandle, PropertySheetPropertyHandle, ObjectBeingCustomized);
				DetailBuilder.EditCategory(DetailCategory).AddCustomBuilder(OverriddenArrayDetails);
			}
			else if (OverriddenPropertyHandle->AsSet().IsValid())
			{
				const FName DetailCategory = OverriddenPropertyHandle->GetDefaultCategoryName();
				TSharedRef<FAudioPropertiesDetailSetBuilder> OverriddenSetDetails = MakeShared<FAudioPropertiesDetailSetBuilder>(OverriddenPropertyHandle, PropertySheetPropertyHandle, ObjectBeingCustomized);
				DetailBuilder.EditCategory(DetailCategory).AddCustomBuilder(OverriddenSetDetails);
			}
			else
			{
				uint32 NumChildren = 0;
				OverriddenPropertyHandle->GetNumChildren(NumChildren);

				if (NumChildren > 0)
				{
					const FName DetailCategory = OverriddenPropertyHandle->GetDefaultCategoryName();
					TSharedRef<FAudioPropertiesDetailNestedNodesBuilder> OverriddenDetails = MakeShared<FAudioPropertiesDetailNestedNodesBuilder>(OverriddenPropertyHandle, PropertySheetPropertyHandle, ObjectBeingCustomized);
					DetailBuilder.EditCategory(DetailCategory).AddCustomBuilder(OverriddenDetails);
				}
				else
				{
					TSharedPtr<SWidget> NameWidget;
					TSharedPtr<SWidget> ValueWidget;
					FDetailWidgetRow PropertyRow;
					OverriddenPropertyRow->GetDefaultWidgets(NameWidget, ValueWidget, PropertyRow);

					OverriddenPropertyRow->CustomWidget(true)
						.NameContent()
						[
							AudioPropertiesDetailsInjectorUtils::CreateOverriddenPropertyNameWidget(OverriddenPropertyHandle, PropertySheetPropertyHandle, ObjectsBeingCustomized)
						]
						.ValueContent()
						[
							SNew(SHorizontalBox)
							.IsEnabled_Lambda([OverriddenPropertyHandle, ObjectBeingCustomized]()
							{
								return !AudioPropertiesDetailsInjectorUtils::ShouldParseProperty(ObjectBeingCustomized, OverriddenPropertyHandle);
							})
							+ SHorizontalBox::Slot()
							[
								ValueWidget.ToSharedRef()
							]
						];

					TAttribute<bool> ResetToDefaultEnabled = TAttribute<bool>::CreateLambda([OverriddenPropertyHandle, ObjectBeingCustomized](){ return !AudioPropertiesDetailsInjectorUtils::ShouldParseProperty(ObjectBeingCustomized, OverriddenPropertyHandle);;});
					OverriddenPropertyRow->OverrideResetToDefault(FResetToDefaultOverride::Create(ResetToDefaultEnabled));
				}
			}
		}
	}
}

void FAudioPropertiesDetailsInjector::BindDetailCustomizationToPropertySheetChanges(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> PropertySheetPropertyHandle)
{
	if (!PropertySheetPropertyHandle->IsValidHandle())
	{
		return;
	}

	TSharedRef<IPropertyUtilities> PropertyUtils = DetailBuilder.GetPropertyUtilities();

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	BindDetailsRefreshToPropertySheetSwaps(DetailBuilder, ObjectsBeingCustomized, PropertySheetPropertyHandle);
	BindDetailsRefreshToPropertySheetChanges(DetailBuilder, ObjectsBeingCustomized, PropertySheetPropertyHandle);

}

void FAudioPropertiesDetailsInjector::BindDetailsRefreshToPropertySheetSwaps(const IDetailLayoutBuilder& DetailBuilder, TArray<TWeakObjectPtr<UObject>>& ObjectsBeingCustomized, TSharedRef<IPropertyHandle> PropertySheetPropertyHandle)
{
	TSharedRef<IPropertyUtilities> PropertyUtils = DetailBuilder.GetPropertyUtilities();

	auto OnMatrixPropertySheetChanged = [PropertyUtils, PropertySheetPropertyHandle, ObjectsBeingCustomized](UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent) {
		
		if(!PropertyChangedEvent.Property)
		{
			return;
		}
		
		if (!ObjectsBeingCustomized.Contains(ObjectBeingModified))
		{
			return;
		}
		
		if (PropertyChangedEvent.Property == PropertySheetPropertyHandle->GetProperty())
		{
			PropertyUtils->ForceRefresh();
		}
	};

	// Preferring this over IPropertyHandle::SetOnPropertyValueChanged because it supports multi edit and editor transactions
	FDelegateHandle Handle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddLambda(OnMatrixPropertySheetChanged);
	OnTargetObjectPropertyChangeDelegateHandles.Add(Handle);
}

void FAudioPropertiesDetailsInjector::BindDetailsRefreshToPropertySheetChanges(IDetailLayoutBuilder& DetailBuilder, TArray<TWeakObjectPtr<UObject>>& ObjectsBeingCustomized, TSharedRef<IPropertyHandle> PropertySheetPropertyHandle)
{
	for (TWeakObjectPtr<UObject>& WeakObjectBeingCustomized : ObjectsBeingCustomized)
	{
		TObjectPtr<UObject> ObjectBeingCustomized = WeakObjectBeingCustomized.Get();

		if (!ObjectBeingCustomized || !ObjectBeingCustomized->IsValidLowLevel())
		{
			continue;
		}

		const UAudioPropertiesSheetAsset* SheetAssetPtr = Cast<UAudioPropertiesSheetAsset>(AudioPropertiesDetailsInjectorUtils::GetUObjectFromPtrPropertyHandle(PropertySheetPropertyHandle, ObjectBeingCustomized));

		if (!SheetAssetPtr || !SheetAssetPtr->IsValidLowLevel())
		{
			continue;
		}

		UAudioPropertiesSheetAsset* NonConstPtrForBinding = const_cast<UAudioPropertiesSheetAsset*>(SheetAssetPtr);

		auto OnPropertiesSheetPostEditChange = [DetailsView = DetailBuilder.GetDetailsViewSharedPtr().ToWeakPtr()](AudioPropertiesSheet::PostEditChangeType InChangeType)
		{
			switch (InChangeType)
			{
			case AudioPropertiesSheet::PostEditChangeType::PropertyBag:
			case AudioPropertiesSheet::PostEditChangeType::ParentChange:
			case AudioPropertiesSheet::PostEditChangeType::PropertiesParser:
				if (TSharedPtr<IDetailsView> DetailsViewPinned = DetailsView.Pin())
				{
					if (DetailsViewPinned->IsParentValid())
					{
						DetailsViewPinned->ForceRefresh();
					}
				}
				break;
			case AudioPropertiesSheet::PostEditChangeType::Other:
				break;
			default:
				break;
			}
		};

		FDelegateHandle PostEditDelegateBinding = NonConstPtrForBinding->OnPostEditChange.AddLambda(OnPropertiesSheetPostEditChange);

		InjectedPropertiesBindings.Add(NonConstPtrForBinding, PostEditDelegateBinding);
	}
}

void FAudioPropertiesDetailsInjector::UnbindFromPropertySheetChanges()
{
	for(const FDelegateHandle& Handle : OnTargetObjectPropertyChangeDelegateHandles)
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(Handle);
	}

	OnTargetObjectPropertyChangeDelegateHandles.Empty();
	
	for (TPair<FObjectKey, FDelegateHandle>& PropertyBinding : InjectedPropertiesBindings)
	{
		if (UObject* BoundObj = PropertyBinding.Key.ResolveObjectPtr())
		{
			if (UAudioPropertiesSheetAsset* SheetAssetPtr = Cast<UAudioPropertiesSheetAsset>(BoundObj))
			{
				SheetAssetPtr->OnPostEditChange.Remove(PropertyBinding.Value);
			}
		}
	}
}
#undef LOCTEXT_NAMESPACE