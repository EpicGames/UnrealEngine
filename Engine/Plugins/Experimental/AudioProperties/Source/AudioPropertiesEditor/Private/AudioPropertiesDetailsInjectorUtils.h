// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioPropertiesSheet.h"
#include "DetailLayoutBuilder.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AudioPropertiesDetailsInjectorUtils"

namespace AudioPropertiesDetailsInjectorUtils
{
	static const FLazyName EditConditionPropertyMetadata = "EditCondition";
	static const FLazyName InlineEditConditionTogglePropertyMetadata = "InlineEditConditionToggle";
	
	inline const UObject* GetUObjectFromPtrPropertyHandle(const TSharedPtr<IPropertyHandle> InPropertyHandle, const UObject* PropertyOwner)
	{
		check(InPropertyHandle)
		check(PropertyOwner)
		
		const FProperty* TargetProperty = InPropertyHandle->GetProperty();
		
		if (!ensure(TargetProperty))
		{
			return nullptr;
		}

		const FName& PropertyName = TargetProperty->GetFName();
		FProperty* PropertyPtr = PropertyOwner->GetClass()->FindPropertyByName(PropertyName);

		if (PropertyPtr)
		{
			if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(PropertyPtr))
			{
				return ObjectProperty->GetObjectPropertyValue(ObjectProperty->ContainerPtrToValuePtr<void>(PropertyOwner));
			}
		}

		return nullptr;
	}

	inline bool ShouldParseProperty(const UObject* PropertyOwner, const TSharedPtr<IPropertyHandle> InPropertyHandle)
	{
		if (!ensureMsgf(PropertyOwner, TEXT("Object should not be null")))
		{
			return false;
		}

		const FProperty* TargetProperty = InPropertyHandle->GetProperty();
		
		if (!TargetProperty)
		{
			return false;
		}

		const IAudioPropertiesSheetAssetUserInterface* PropertySheetAssetUser = Cast<IAudioPropertiesSheetAssetUserInterface>(PropertyOwner);

		if (!ensureMsgf(PropertySheetAssetUser, TEXT("Implement IAudioPropertiesSheetAssetUserInterface on objects using PropertySheets to support local overrides")))
		{
			return false;
		}

		return PropertySheetAssetUser->ShouldParseProperty(*TargetProperty);
	}

	inline void HandleLocalPropertyOverrideChange(UObject* PropertyOwner, const bool bIsPropertyOverriden, const TSharedPtr<IPropertyHandle> TargetPropertyHandle, const TSharedPtr<IPropertyHandle> PropertySheetHandle)
	{
		if (!ensureMsgf(PropertyOwner, TEXT("Object should not be null")))
		{
			return;
		}

		const FProperty* TargetProperty = TargetPropertyHandle->GetProperty();
		
		if (!TargetProperty)
		{
			return;
		}

		IAudioPropertiesSheetAssetUserInterface* PropertySheetAssetUser = Cast<IAudioPropertiesSheetAssetUserInterface>(PropertyOwner);

		if (!ensureMsgf(PropertySheetAssetUser, TEXT("Implement IAudioPropertiesSheetAssetUserInterface on objects using PropertySheets to support local overrides")))
		{
			return;
		}

		FScopedTransaction Transaction(FText::Format(LOCTEXT("OnCheckStateChanged", "Changed Local Override for property {0}"), FText::FromName(TargetProperty->GetFName())));
		PropertyOwner->Modify();

		if (bIsPropertyOverriden)
		{
			PropertySheetAssetUser->IgnorePropertyParsing(*TargetProperty);
		}
		else
		{
			PropertySheetAssetUser->AllowPropertyParsing(*TargetProperty);

			const UAudioPropertiesSheetAsset* SheetAssetPtr = Cast<UAudioPropertiesSheetAsset>(GetUObjectFromPtrPropertyHandle(PropertySheetHandle, PropertyOwner));

			if (!SheetAssetPtr || !SheetAssetPtr->IsValidLowLevel())
			{
				return;
			}

			SheetAssetPtr->CopyToObjectProperties(PropertyOwner);
		}
	}

	inline TSharedRef<SWidget> GetSingleEditPropertyNameWidget(TSharedPtr<IPropertyHandle> PropertyHandle, const TSharedPtr<IPropertyHandle> PropertySheetHandle, const TArray<TWeakObjectPtr<UObject>>& ObjectsBeingCustomized)
	{
		checkf(ObjectsBeingCustomized.Num() == 1, TEXT("GetMultiEditPropertyNameWidget should be used when editing multiple objects"))

		const UAudioPropertiesSheetAsset* SheetAssetPtr = Cast<UAudioPropertiesSheetAsset>(AudioPropertiesDetailsInjectorUtils::GetUObjectFromPtrPropertyHandle(PropertySheetHandle, ObjectsBeingCustomized[0].Get()));
		FText PropertiesSheetName;

		if (SheetAssetPtr)
		{
			PropertiesSheetName = FText::FromName(SheetAssetPtr->GetFName());
		}

		FText TooltipFText = FText::Format(LOCTEXT("OverriddenAudioPropertyTooltip", "Property Overridden by {0}"), PropertiesSheetName);

		TSharedRef<SToolTip> NameWidgetTooltip = SNew(SToolTip)
		[
			SNew(STextBlock)
			.Text(TooltipFText)
		];


		TSharedRef<SWidget> PropertyNameWidget =  SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([PropertyHandle, OwnerObject = ObjectsBeingCustomized[0]]()
				{
					return AudioPropertiesDetailsInjectorUtils::ShouldParseProperty(OwnerObject.Get(), PropertyHandle) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
				})
				.OnCheckStateChanged_Lambda([PropertyHandle, PropertySheetHandle, OwnerObject = ObjectsBeingCustomized[0]](ECheckBoxState NewState)
				{
					const bool bIsOverridden = NewState == ECheckBoxState::Checked;
					HandleLocalPropertyOverrideChange(OwnerObject.Get(), bIsOverridden, PropertyHandle, PropertySheetHandle);
				})
			]
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([PropertyHandle, OwnerObject = ObjectsBeingCustomized[0]]()
				{
					if (AudioPropertiesDetailsInjectorUtils::ShouldParseProperty(OwnerObject.Get(), PropertyHandle))
					{
						return FText::Format(LOCTEXT("OverriddenAudioPropertyDisplayName", "[AudioProperty Override] {0}"), PropertyHandle->GetPropertyDisplayName());
					}

					return FText::Format(LOCTEXT("OverriddenAudioLocalPropertyDisplayName", "[Local] {0}"), PropertyHandle->GetPropertyDisplayName());
				})
				.ToolTip(NameWidgetTooltip)
				.IsEnabled_Lambda([PropertyHandle, OwnerObject = ObjectsBeingCustomized[0]]()
				{
					return !AudioPropertiesDetailsInjectorUtils::ShouldParseProperty(OwnerObject.Get(), PropertyHandle);
				})
			];


		return PropertyNameWidget;
	}

	inline TSharedRef<SWidget> GetMultiEditPropertyNameWidget(TSharedPtr<IPropertyHandle> PropertyHandle, const TSharedPtr<IPropertyHandle> PropertySheetHandle, const TArray<TWeakObjectPtr<UObject>>& ObjectsBeingCustomized)
	{
		checkf(ObjectsBeingCustomized.Num() > 0, TEXT("GetSingleEditPropertyNameWidget should be used when editing a single object"))

		FText TooltipFText = LOCTEXT("OverriddenAudioPropertyByOneOrMoreTooltip", "Property Overridden by one or more properties sheet. Use drop down to select local overrides");

		//create a drop-down menu that allows to check which assets in the matrix have local overrides for a specific property
		TSharedRef<SWidget> PropertyNameWidget = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.0f)
				[
					SNew(SComboButton)
						.OnGetMenuContent(FOnGetContent::CreateLambda([PropertyHandle, PropertySheetHandle, ObjectsBeingCustomized]()
						{
							// Create a vertical box to hold the check-boxes
							TSharedPtr<SVerticalBox> VerticalBox;

							SAssignNew(VerticalBox, SVerticalBox);

							// Iterate over the objects and create a checkbox for each
							for (TWeakObjectPtr<UObject> WeakObject : ObjectsBeingCustomized)
							{
								if (WeakObject.IsValid())
								{
									// Add an entry to the vertical box for each object
									VerticalBox->AddSlot()
										.AutoHeight()
										[
											SNew(SHorizontalBox)
											+ SHorizontalBox::Slot()
											.AutoWidth()
											.VAlign(VAlign_Center)
											[
												SNew(SCheckBox)
												.IsChecked_Lambda([PropertyHandle, WeakObject]()
												{
													return AudioPropertiesDetailsInjectorUtils::ShouldParseProperty(WeakObject.Get(), PropertyHandle) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
												})
												.OnCheckStateChanged_Lambda([PropertyHandle, PropertySheetHandle, WeakObject](ECheckBoxState NewState)
												{
													const bool bIsOverridden = NewState == ECheckBoxState::Checked;
													HandleLocalPropertyOverrideChange(WeakObject.Get(), bIsOverridden, PropertyHandle, PropertySheetHandle);
												})
												]
												+ SHorizontalBox::Slot()
												.Padding(5.0f, 0.0f)
												.VAlign(VAlign_Center)
												[
													SNew(STextBlock)
														.Text(FText::FromString(WeakObject->GetName()))
												]
										];
								}
							}

							return SNew(SBorder)
									.BorderImage(FCoreStyle::Get().GetBrush("Menu.Background"))
									[
										VerticalBox.ToSharedRef()
									];
						}))
						.ButtonContent()
							[
								SNew(STextBlock)
								.Text(PropertyHandle->GetPropertyDisplayName())
								.Font(IDetailLayoutBuilder::GetDetailFont())
							]
				];

		return PropertyNameWidget;
	}


	inline TSharedRef<SWidget> CreateOverriddenPropertyNameWidget(TSharedPtr<IPropertyHandle> PropertyHandle,  const TSharedPtr<IPropertyHandle> PropertySheetHandle, const TArray<TWeakObjectPtr<UObject>>& ObjectsBeingCustomized)
	{
		check(ObjectsBeingCustomized.Num() > 0)
		const bool bMultiEdit = ObjectsBeingCustomized.Num() > 1;

		if (!bMultiEdit)
		{
			return GetSingleEditPropertyNameWidget(PropertyHandle, PropertySheetHandle, ObjectsBeingCustomized);
		}
		else
		{
			return GetMultiEditPropertyNameWidget(PropertyHandle, PropertySheetHandle, ObjectsBeingCustomized);
		}
	}
}
#undef LOCTEXT_NAMESPACE 
