// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesEditorLogCategory.h"

DEFINE_LOG_CATEGORY(LogAudioPropertiesEditor);