// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesEditorModule.h"

#include "AssetToolsModule.h"
#include "AudioPropertiesSheetBuilderInstantiator.h"
#include "AudioPropertiesDetailsInjector.h"
#include "AudioPropertiesSheetAssetDetails.h"
#include "AudioPropertiesSheetDetails.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

namespace AudioPropertiesEditorModule
{
	static const FName AssetToolsModuleName = TEXT("AssetTools");
}

void FAudioPropertiesEditorModule::StartupModule()
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>(AudioPropertiesEditorModule::AssetToolsModuleName).Get();
	
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout("AudioPropertiesSheet", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAudioPropertiesSheetDetails::MakeInstance));
	PropertyModule.NotifyCustomizationModuleChanged();

	PropertyModule.RegisterCustomClassLayout("AudioPropertiesSheetAsset", FOnGetDetailCustomizationInstance::CreateStatic(&FAudioPropertiesSheetAssetDetails::MakeInstance));

	DetailsInjectorBuilder = MakeShared<FAudioPropertiesDetailsInjectorBuilder>();

	IModularFeatures::Get().RegisterModularFeature(AudioPropertiesDetailsInjector::BuilderModularFeatureName, DetailsInjectorBuilder.Get());

	
	PropertiesSheetBuilder = MakeShared<FAudioPropertiesSheetBuilderInstantiator>();
	PropertiesSheetBuilder->ExtendContentBrowserSelectionMenu();
}

void FAudioPropertiesEditorModule::ShutdownModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.UnregisterCustomPropertyTypeLayout("AudioPropertiesSheet");
	PropertyModule.NotifyCustomizationModuleChanged();
	
	IModularFeatures::Get().UnregisterModularFeature(AudioPropertiesDetailsInjector::BuilderModularFeatureName, DetailsInjectorBuilder.Get());
	DetailsInjectorBuilder.Reset();
}

IMPLEMENT_MODULE(FAudioPropertiesEditorModule, AudioPropertiesEditor);