// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesSheetAssetBuilder.h"

#include "AudioPropertiesSheet.h"
#include "AudioPropertiesSheetAssetFactory.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AudioPropertiesUtils.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "UObject/PropertyAccessUtil.h"

#define LOCTEXT_NAMESPACE "AudioPropertiesSheetBuilderWidget"

void FAudioPropertiesSheetAssetBuilder::BuildPropertySheetFromPropertyDataArray(const UObject* SourceObject, const UAudioPropertiesSheetAsset* ParentSheet, TArrayView<AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr> InPropertyData, AudioPropertiesSheetAssetBuilder::ESourceObjectParsingType ParsingType /*= AudioPropertiesSheetAssetBuilder::ESourceObjectParsingType::UObject*/)
{
	const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

	FString BasePackageName = SourceObject ? SourceObject->GetPackage()->GetPathName().Append(TEXT("_GeneratedPropertySheet")) : TEXT("/Game/GeneratedPropertySheet");
	FString OutNewPackageName;
	FString OutNewAssetName;
	AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, TEXT(""), OutNewPackageName, OutNewAssetName);

	const TSharedRef<SDlgPickAssetPath> NewAssetDialog =
		SNew(SDlgPickAssetPath)
		.Title(LOCTEXT("CreateAudioPropertiesSheet", "Create Audio Properties Sheet"))
		.DefaultAssetPath(FText::FromString(OutNewPackageName));

	if (NewAssetDialog->ShowModal() == EAppReturnType::Cancel)
	{
		return;
	}

	const FString PackageName = NewAssetDialog->GetFullAssetPath().ToString();
	const FName AssetName = FName(*NewAssetDialog->GetAssetName().ToString());

	UPackage* const Package = CreatePackage(*PackageName);
		
	// Create Property Sheet Object
	UObject* NewPropertySheetObject = nullptr;
		
	if (UAudioPropertiesSheetAssetFactory* const SheetFactory = Cast<UAudioPropertiesSheetAssetFactory>(NewObject<UFactory>(GetTransientPackage(), UAudioPropertiesSheetAssetFactory::StaticClass())))
	{
		NewPropertySheetObject = SheetFactory->FactoryCreateNew(SheetFactory->GetSupportedClass(), Package, AssetName, RF_Public | RF_Standalone, nullptr, GWarn);
	}

	if (!NewPropertySheetObject)
	{
		return;
	}

	// Copy selected properties
	UAudioPropertiesSheetAsset* const NewPropertySheetAsset = CastChecked<UAudioPropertiesSheetAsset>(NewPropertySheetObject);

	if (!NewPropertySheetAsset)
	{
		return;
	}

	NewPropertySheetAsset->PropertiesSheet.Parent = ParentSheet;

	FInstancedPropertyBag& SheetPropertyBag = NewPropertySheetAsset->PropertiesSheet.Properties;

	for (const AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr& PropertyRequest : InPropertyData)
	{
		if (!PropertyRequest->bInheritProperty)
		{
			continue;
		}

		const FProperty* PropertyToInherit = PropertyRequest->CachedProperty;

		if (!PropertyToInherit)
		{
			continue;
		}

		const FName& PropertyName = PropertyToInherit->GetFName();

		//We create the desc locally first to remove any inherited metadata, we only want matching name and type on the sheet
		FPropertyBagPropertyDesc PropertyDesc(PropertyName, PropertyToInherit);
		PropertyDesc.MetaData.Empty();
		SheetPropertyBag.AddProperties({PropertyDesc});
		
		const FPropertyBagPropertyDesc* CreatedPropertyDesc = SheetPropertyBag.FindPropertyDescByName(PropertyName);
		
		if (!CreatedPropertyDesc)
		{
			UE_LOGF(LogInit, Warning, "Failed to create property %ls from request", *PropertyName.ToString());
			continue;
		}

		if(!PropertyRequest->bInheritValue)
		{
			continue;
		}

		FStructView SheetPropertyBagView = SheetPropertyBag.GetMutableValue();
		void* TargetPropertyAddress = CreatedPropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>(SheetPropertyBagView.GetMemory());
		
		if (ParsingType == AudioPropertiesSheetAssetBuilder::ESourceObjectParsingType::UObject)
		{
			const void* SourcePropertyAddress = PropertyToInherit->ContainerPtrToValuePtr<void>(SourceObject);
			const bool bSuccessfulCopy = PropertyAccessUtil::CopyCompletePropertyValue(PropertyToInherit, SourcePropertyAddress, CreatedPropertyDesc->CachedProperty, TargetPropertyAddress);
		}
		else if (ParsingType == AudioPropertiesSheetAssetBuilder::ESourceObjectParsingType::PropertySheet)
		{
			const UAudioPropertiesSheetAsset* SourceAsPropertySheet = Cast<UAudioPropertiesSheetAsset>(SourceObject);

			if (!ensureAlwaysMsgf(SourceAsPropertySheet, TEXT("Trying to copy properties in property sheet mode from a non property sheet asset. Property copy will be skipped")))
			{
				continue;
			}

			auto OnLeafPropertyVisited = [&CreatedPropertyDesc, &TargetPropertyAddress, PropertyName](const FPropertyBagPropertyDesc& PropertyDesc, const FAudioPropertiesSheet& OwningSheet)
			{
				if (PropertyDesc.Name == PropertyName)
				{
					const void* SourcePropertyCointainerAddress = CreatedPropertyDesc->CachedProperty->ContainerPtrToValuePtr<void>(OwningSheet.Properties.GetValue().GetMemory());
					const bool bSuccessfulCopy = PropertyAccessUtil::CopyCompletePropertyValue(PropertyDesc.CachedProperty, SourcePropertyCointainerAddress, CreatedPropertyDesc->CachedProperty, TargetPropertyAddress);
				}
			};

			AudioPropertiesUtils::VisitLeafMostProperties(*SourceAsPropertySheet, OnLeafPropertyVisited);
		}
		else
		{
			checkNoEntry()
		}
	}

	NewPropertySheetAsset->PropertiesSheet.ReconcileProperties();

	FAssetRegistryModule::AssetCreated(NewPropertySheetObject);

	Package->GetOutermost()->MarkPackageDirty();
}

#undef LOCTEXT_NAMESPACE