// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class UAudioPropertiesSheetAsset;
class UObject;
class FProperty;

namespace AudioPropertiesSheetAssetBuilder
{
	struct FPropertyRequest
	{
		const FProperty* CachedProperty;
		bool bInheritProperty;
		bool bInheritValue;

		FPropertyRequest(const FProperty* InCachedProperty,
			bool bInInheritProperty = false,
			bool bInInheritValue = false)
			: CachedProperty(InCachedProperty)
			, bInheritProperty(bInInheritProperty)
			, bInheritValue(bInInheritValue)
		{}
	};

	enum class ESourceObjectParsingType :uint8
	{
		UObject, //To parse UProperties found on the source UObject 
		PropertySheet //To parse from the property sheet contained on a source UAudioPropertiesSheetAsset 
	};

	using FPropertyRequestPtr = TSharedPtr<FPropertyRequest>;
}

class FAudioPropertiesSheetAssetBuilder
{
public:
	static void BuildPropertySheetFromPropertyDataArray(const UObject* SourceObject, const UAudioPropertiesSheetAsset* ParentSheet, TArrayView<AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr> InPropertyData, AudioPropertiesSheetAssetBuilder::ESourceObjectParsingType ParsingType = AudioPropertiesSheetAssetBuilder::ESourceObjectParsingType::UObject);
};