// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesSheetAssetDetails.h"

#include "AudioPropertiesParserBase.h"
#include "AudioPropertiesSheet.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FAudioPropertiesSheetAssetDetails"

namespace AudioPropertiesSheetAssetDetailsPrivate
{
	TSharedRef<SToolTip> CreatePathTooltip(const UObject* TargetObject)
	{
		FText OwnerTooltip = LOCTEXT("AudioPropertiesSheetAssetDetailsPathTooltip", "None");

		if (TargetObject)
		{
			if (TargetObject->GetPackage())
			{
				OwnerTooltip = FText::FromString(TargetObject->GetPackage()->GetPathName());

			}
		}

		return SNew(SToolTip)
			[
				SNew(STextBlock)
				.Text(OwnerTooltip)
			];
	}
}

TSharedRef<IDetailCustomization> FAudioPropertiesSheetAssetDetails::MakeInstance()
{
	return MakeShareable(new FAudioPropertiesSheetAssetDetails);
}

void FAudioPropertiesSheetAssetDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	BuildPropertiesParserRow(DetailBuilder);
}

void FAudioPropertiesSheetAssetDetails::BuildPropertiesParserRow(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	
	//We should have only one object being customized for this details view
	if (!ensureMsgf((Objects.Num() == 1 && Objects[0].IsValid()), TEXT("Expected one valid object in the details view")))
	{
		return;
	}

	UAudioPropertiesSheetAsset* SheetAsset = CastChecked<UAudioPropertiesSheetAsset>(Objects[0].Get());

	if (!SheetAsset)
	{
		return;
	}

	//To keep the custom parser rows in place we group them into a category because...
	//...we can sort Categories but not rows 
	IDetailCategoryBuilder& DefaultCategory = DetailBuilder.EditCategory("Default");
	DefaultCategory.SetSortOrder(0);
	IDetailCategoryBuilder& ParserCategory = DetailBuilder.EditCategory("Parser");
	ParserCategory.SetSortOrder(1);

	//Hide and read the local parser property so it stays on top of the category
	TSharedRef<IPropertyHandle> LocalParserPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAudioPropertiesSheetAsset, PropertiesParser));
	DetailBuilder.HideProperty(LocalParserPropertyHandle);
	ParserCategory.AddProperty(LocalParserPropertyHandle);

	//if we have a valid parser, we don't need to show an inherited one
	if (SheetAsset->PropertiesParser)
	{
		return;
	}

	AudioPropertiesSheet::ParserParentPair ClosestParser = SheetAsset->FindClosestParserInInheritanceTree();
	TObjectPtr<const UAudioPropertiesParserBase> InheritedParser = ClosestParser.Key;
	const UAudioPropertiesSheetAsset* InheritedParserOwner = ClosestParser.Value;

	if (!(InheritedParser && InheritedParserOwner))
	{
		return;
	}

	IDetailPropertyRow* InheritedParserPropertyRow = ParserCategory.AddExternalObjectProperty({ const_cast<UAudioPropertiesSheetAsset*>(InheritedParserOwner) }, GET_MEMBER_NAME_CHECKED(UAudioPropertiesSheetAsset, PropertiesParser));
	
	TSharedPtr<SWidget> InheritedParserDefaultNameWidget;
	TSharedPtr<SWidget> InheritedParserDefaultValueWidget;
	InheritedParserPropertyRow->GetDefaultWidgets(InheritedParserDefaultNameWidget, InheritedParserDefaultValueWidget);
	
	FString ParserOwnerName;
	InheritedParserOwner->GetName(ParserOwnerName);
	const FString ParentAssetPropertyString = FString::Printf(TEXT("[%s] Inherited Parser"), *ParserOwnerName);
	FText InheritedParserNameText = FText::FromString(ParentAssetPropertyString);

	TSharedRef<SWidget> InheritedParserNameWidget = SNew(STextBlock)
		.Text(InheritedParserNameText)
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		.ToolTip(AudioPropertiesSheetAssetDetailsPrivate::CreatePathTooltip(InheritedParserOwner));

	InheritedParserPropertyRow->CustomWidget(true)
		.NameContent()[
			InheritedParserNameWidget
		]
		.ValueContent()[
			InheritedParserDefaultValueWidget.ToSharedRef()
		];

	//we don't want to allow editing of the inherited parser
	InheritedParserPropertyRow->IsEnabled(false);
}

#undef LOCTEXT_NAMESPACE