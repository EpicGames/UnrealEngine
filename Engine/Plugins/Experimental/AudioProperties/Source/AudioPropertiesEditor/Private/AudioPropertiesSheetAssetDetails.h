// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class FAudioPropertiesSheetAssetDetails : public IDetailCustomization
{
	public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	/** Allows to select the local parser and shows the inherited one in case local is null */
	void BuildPropertiesParserRow(IDetailLayoutBuilder& DetailBuilder);
};