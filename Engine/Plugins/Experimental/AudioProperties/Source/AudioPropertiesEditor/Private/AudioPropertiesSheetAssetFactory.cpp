// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesSheetAssetFactory.h"

#include "AudioPropertiesSheet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioPropertiesSheetAssetFactory)

UAudioPropertiesSheetAssetFactory::UAudioPropertiesSheetAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAudioPropertiesSheetAsset::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}


UObject* UAudioPropertiesSheetAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAudioPropertiesSheetAsset>(InParent, Name, Flags);
}
