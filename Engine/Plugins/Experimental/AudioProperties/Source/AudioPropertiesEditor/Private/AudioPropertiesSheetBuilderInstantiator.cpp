// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesSheetBuilderInstantiator.h"

#include "Algo/AnyOf.h"
#include "AudioPropertiesEditorLogCategory.h"
#include "AudioPropertiesSheet.h"
#include "AssetDefinitionRegistry.h"
#include "AssetDefinition_AudioPropertiesSheetAsset.h"
#include "AssetTypeActions/AssetDefinition_SoundBase.h"
#include "ContentBrowserMenuContexts.h"
#include "Engine/AssetManager.h"
#include "Engine/StreamableManager.h"
#include "Framework/Application/SlateApplication.h"
#include "SAudioPropertiesSheetBuilderWidget.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "AudioPropertiesSheetBuilder"

namespace AudioPropertiesBuilderPrivate
{
	auto IsPropertySheet = [](const FAssetData& AssetData) { return AssetData.IsInstanceOf<UAudioPropertiesSheetAssetBase>(); };
}

void FAudioPropertiesSheetBuilderInstantiator::ExtendContentBrowserSelectionMenu()
{
	if (!UToolMenus::IsToolMenuUIEnabled())
	{
		return;
	}
	
	const UAssetDefinitionRegistry* DefRegistry = UAssetDefinitionRegistry::Get();
	check(DefRegistry);

	TArray<TObjectPtr<UAssetDefinition>> AssetDefinitions = DefRegistry->GetAllAssetDefinitions();
	for (TObjectPtr<UAssetDefinition> AssetDef : AssetDefinitions)
	{
		if (UAssetDefinition_SoundBase* SoundAssetDef = Cast<UAssetDefinition_SoundBase>(AssetDef.Get()))
		{
			FToolMenuSection* Section = SoundAssetDef->FindSoundContextMenuSection("Sound");
			check(Section);

			Section->AddDynamicEntry("AudioPropertiesSheet_CreatePropertiesSheetBuilder", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>())
				{
					const TAttribute<FText> Label = LOCTEXT("AudioPropertiesSheetBuilderMenu_Label", "Open Property Sheet Builder");
					const TAttribute<FText> ToolTip = LOCTEXT("AudioPropertiesSheetBuilderMenu_Tooltip", "Opens a utility for creating property sheets with properties from the selected objects.");
					const FSlateIcon Icon = FSlateIcon();
					const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateSP(this, &FAudioPropertiesSheetBuilderInstantiator::ExecuteCreateBuilderWidget);
		
					InSection.AddMenuEntry("AudioPropertiesSheetBuilderMenu_Label", Label, ToolTip, Icon, UIAction);
					
				}
			}));
		}
		else if (UAssetDefinition_AudioPropertiesSheetAsset* AudioPropertySheetAssetDef = Cast<UAssetDefinition_AudioPropertiesSheetAsset>(AssetDef.Get()))
		{
			FToolMenuSection* Section = AudioPropertySheetAssetDef->FindContextMenuSection("Builder");
			check(Section);
	
			Section->AddDynamicEntry("AudioPropertiesSheet_CreateChildPropertySheet", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>())
				{
					if (Algo::AnyOf(Context->SelectedAssets, AudioPropertiesBuilderPrivate::IsPropertySheet))
					{
						const TAttribute<FText> Label = LOCTEXT("AudioPropertiesSheetChildBuilderMenu_Label", "Open Property Sheet Child Builder");
						const TAttribute<FText> ToolTip = LOCTEXT("AudioPropertiesSheetChildBuilderMenu_Tooltip", "Opens a utility for creating a child of this property sheet");
						const FSlateIcon Icon = FSlateIcon();
						const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateSP(this, &FAudioPropertiesSheetBuilderInstantiator::ExecuteCreateBuilderWidget);
	
						InSection.AddMenuEntry("AudioPropertiesSheetBuilderMenu_Label", Label, ToolTip, Icon, UIAction);
					}
				}
			}));
		}
	}	
}

void FAudioPropertiesSheetBuilderInstantiator::ExecuteCreateBuilderWidget(const FToolMenuContext& MenuContext)
{
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
	{
		TArray<TSoftObjectPtr<UObject>> SelectedObjects = Context->GetSelectedAssetSoftObjects<UObject>();
		CreateBuilderWidget(SelectedObjects);
	}
}

void FAudioPropertiesSheetBuilderInstantiator::CreateBuilderWidget(TArrayView<TSoftObjectPtr<UObject>> SourceObjects)
{
	for (TSoftObjectPtr<UObject> SourceObject : SourceObjects)
	{
		UE_LOGF(LogAudioPropertiesEditor, Log, "Loading Object %ls to open property sheet builder", *SourceObject->GetPathName())
		FStreamableDelegate LibDelegate = FStreamableDelegate::CreateSP(this, &FAudioPropertiesSheetBuilderInstantiator::SourceObjectAsyncLoadComplete, SourceObject);
		UAssetManager::GetStreamableManager().RequestAsyncLoad(SourceObject.ToSoftObjectPath(), MoveTemp(LibDelegate));
	}
}

void FAudioPropertiesSheetBuilderInstantiator::SourceObjectAsyncLoadComplete(TSoftObjectPtr<UObject> LoadedObject)
{
	if (!LoadedObject.IsValid())
	{
		return;
	}

	UE_LOGF(LogAudioPropertiesEditor, Log, "Opening property sheet builder with source %ls", *LoadedObject->GetPathName())
	
	TObjectPtr<UObject> LoadedSource = LoadedObject.Get();
	const UClass* SourceClass = LoadedSource.GetClass();
	
	TSharedRef<SWindow> PropertiesWindow = SNew(SWindow)
	.Title(FText::FromString("Audio Properties Sheet Builder"))
	.ClientSize(FVector2D(800, 600))
	.SupportsMinimize(false)
	.SupportsMaximize(false);

	PropertiesWindow->SetContent(
		SNew(SAudioPropertiesSheetBuilderWidget)
			.SourceObject(LoadedSource.Get())
			.bSourceIsParent(SourceClass && SourceClass->IsChildOf(UAudioPropertiesSheetAsset::StaticClass()))
	);

	FSlateApplication::Get().AddWindow(PropertiesWindow);
}


#undef LOCTEXT_NAMESPACE
