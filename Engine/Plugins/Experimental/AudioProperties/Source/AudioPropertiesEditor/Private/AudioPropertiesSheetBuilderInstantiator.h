// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"

class UObject;
struct FToolMenuContext;

class FAudioPropertiesSheetBuilderInstantiator : public TSharedFromThis<FAudioPropertiesSheetBuilderInstantiator>
{
public:
	void ExtendContentBrowserSelectionMenu();

private:
	void ExecuteCreateBuilderWidget(const FToolMenuContext& MenuContext);
	void CreateBuilderWidget(TArrayView<TSoftObjectPtr<UObject>> SourceObjects);
	void SourceObjectAsyncLoadComplete(TSoftObjectPtr<UObject> LoadedObject);
};
