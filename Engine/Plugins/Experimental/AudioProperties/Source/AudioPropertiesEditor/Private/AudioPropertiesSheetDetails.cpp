// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesSheetDetails.h"

#include "AudioPropertiesEditorLogCategory.h"
#include "AudioPropertiesSheet.h"
#include "AudioPropertiesUtils.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailChildrenBuilder.h"
#include "InstancedAudioPropertyBagDetails.h"
#include "IPropertyChangeListener.h"
#include "IPropertyUtilities.h"
#include "PropertyBagDetails.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FAudioPropertiesSheetDetails"

namespace AudioPropertiesSheetDetailsPrivate
{
	const UObject* GetUObjectFromParentPtrPropertyHandle(const TSharedPtr<IPropertyHandle> InPropertyHandle)
	{
		check(InPropertyHandle)

		if (FProperty* PropertyPtr = InPropertyHandle->GetProperty())
		{
			if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(PropertyPtr))
			{
				void* ObjPropertyPtr = nullptr;
				InPropertyHandle->GetValueData(ObjPropertyPtr);
				if (ObjPropertyPtr)
				{
					TObjectPtr<const UObject>* ObjPtr = (TObjectPtr<const UObject>*)ObjPropertyPtr;
					return ObjPtr->Get();
				}
			}
		}

		return nullptr;

	}

	static const FName ParentPropertyName = GET_MEMBER_NAME_CHECKED(FAudioPropertiesSheet, Parent);
	static const FName PropertyBagPropertyName = GET_MEMBER_NAME_CHECKED(FAudioPropertiesSheet, Properties);
}

TSharedRef<IPropertyTypeCustomization> FAudioPropertiesSheetDetails::MakeInstance()
{
	return MakeShareable(new FAudioPropertiesSheetDetails);
}

void FAudioPropertiesSheetDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

void FAudioPropertiesSheetDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	FAudioPropertiesSheet* PropertiesSheetStruct = nullptr; 
	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);
	check(OuterObjects.Num() > 0 && OuterObjects[0]) //multi edit not supported yet

	UAudioPropertiesSheetAsset* Owner = Cast<UAudioPropertiesSheetAsset>(OuterObjects[0]);
	if (Owner)
	{
		PropertiesSheetStruct = &Owner->PropertiesSheet;

		const TSharedRef<FAudioPropertiesSheetBagNodeBuilder> BagNodeBuilder = MakeShareable(new FAudioPropertiesSheetBagNodeBuilder(InPropertyHandle, InCustomizationUtils.GetPropertyUtilities(), PropertiesSheetStruct));
		InChildrenBuilder.AddCustomBuilder(BagNodeBuilder);

		const bool bIsLeaf = true;
		const TSharedRef<FAudioPropertiesSheetAssetTreeNodeBuilder> AssetTreeNodeBuilder = MakeShareable(new FAudioPropertiesSheetAssetTreeNodeBuilder(InPropertyHandle, InPropertyHandle->GetChildHandle(AudioPropertiesSheetDetailsPrivate::ParentPropertyName), PropertiesSheetStruct, bIsLeaf));
		InChildrenBuilder.AddCustomBuilder(AssetTreeNodeBuilder);
	}
}

FAudioPropertiesSheetBagNodeBuilder::FAudioPropertiesSheetBagNodeBuilder(TSharedPtr<IPropertyHandle> InPropertySheetStructHandle, TSharedPtr<IPropertyUtilities> InPropertyUtils, FAudioPropertiesSheet* InLeafMostSheet /*= nullptr*/)
	: CachedSheetStructHandle(InPropertySheetStructHandle)
	, CachedPropertyUtils(InPropertyUtils)
	, LeafMostSheet(InLeafMostSheet)
{
	if (CachedSheetStructHandle)
	{
		ParentPropertyHandle = CachedSheetStructHandle->GetChildHandle(AudioPropertiesSheetDetailsPrivate::ParentPropertyName);
	}
}

void FAudioPropertiesSheetBagNodeBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	NodeRow.NameContent()[BuildHeaderRowName()]
			.ValueContent()[BuildHeaderRowValue()]
			.ShouldAutoExpand(true);
}

FName FAudioPropertiesSheetBagNodeBuilder::GetName() const
{
	static const FName AudioPropertiesSheetBagNodeName("AudioPropertiesSheetBagNode");
	return AudioPropertiesSheetBagNodeName;
}

void FAudioPropertiesSheetBagNodeBuilder::OnParentAssetPropertyOverrideChange(const FProperty& TargetProperty, bool bIsOverridden)
{
	if (CachedPropertyUtils) 
	{
		CachedPropertyUtils->ForceRefresh();
	}
}

void FAudioPropertiesSheetBagNodeBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (TSharedPtr<IPropertyHandle> LocalPropertyBagHandle = CachedSheetStructHandle->GetChildHandle(AudioPropertiesSheetDetailsPrivate::PropertyBagPropertyName))
	{
		const bool bIsLeafBag = true;

		const TSharedRef<FInstancedAudioPropertyBagDetails> LocalBagDetails = MakeShareable(new FInstancedAudioPropertyBagDetails(LocalPropertyBagHandle, CachedPropertyUtils, nullptr, LeafMostSheet, bIsLeafBag));
		ChildrenBuilder.AddCustomBuilder(LocalBagDetails);
	}

	AddParentTreePropertyBags(ChildrenBuilder);
}

void FAudioPropertiesSheetBagNodeBuilder::AddParentTreePropertyBags(IDetailChildrenBuilder& ChildrenBuilder)
{
	const UObject* ParentAsset = AudioPropertiesSheetDetailsPrivate::GetUObjectFromParentPtrPropertyHandle(ParentPropertyHandle);

	while (ParentAsset)
	{
		TArray<UObject*> ParentObjects;
		ParentObjects.Add(const_cast<UObject*>(ParentAsset));

		UAudioPropertiesSheetAsset* ParentAsSheet = Cast<UAudioPropertiesSheetAsset>(ParentObjects[0]);
				
		if (ParentAsSheet)
		{
			auto ParentObjectsRow = ChildrenBuilder.AddExternalObjectProperty(ParentObjects, TEXT("PropertiesSheet"));
			auto ParentRowPropertyHandle = ParentObjectsRow->GetPropertyHandle();
			ParentObjectsRow->Visibility(EVisibility::Hidden);

			if (TSharedPtr<IPropertyHandle> PropertyBagHandle = ParentRowPropertyHandle->GetChildHandle(AudioPropertiesSheetDetailsPrivate::PropertyBagPropertyName))
			{
				const bool bIsLeafBag = false;

				TSharedRef<FInstancedAudioPropertyBagDetails> InheritedBagDetails = MakeShareable(new FInstancedAudioPropertyBagDetails(PropertyBagHandle, CachedPropertyUtils, ParentAsSheet, LeafMostSheet, bIsLeafBag));
				ChildrenBuilder.AddCustomBuilder(InheritedBagDetails);
			
				ParentAsSheet->PropertiesSheet.OnAudioPropertyOverrideChange.AddSP( this, &FAudioPropertiesSheetBagNodeBuilder::OnParentAssetPropertyOverrideChange );

			}

			ParentAsset = ParentAsSheet->PropertiesSheet.Parent;

		}
		else
		{
			ParentAsset = nullptr;
		}		
	}
}

TSharedRef<SWidget> FAudioPropertiesSheetBagNodeBuilder::BuildHeaderRowName() const
{
	FText HeaderValueText =  LOCTEXT("AudioPropertiesSheetBagNodeHeaderName", "Properties");

	return SNew(STextBlock)
		.Text(HeaderValueText)
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

TSharedRef<SWidget> FAudioPropertiesSheetBagNodeBuilder::BuildHeaderRowValue()
{
	if (TSharedPtr<IPropertyHandle> LocalPropertyBagHandle = CachedSheetStructHandle->GetChildHandle(AudioPropertiesSheetDetailsPrivate::PropertyBagPropertyName))
	{
		return FPropertyBagDetails::MakeAddPropertyWidget(LocalPropertyBagHandle, CachedPropertyUtils).ToSharedRef();
	}

	return SNew(STextBlock);
}

FAudioPropertiesSheetAssetTreeNodeBuilder::FAudioPropertiesSheetAssetTreeNodeBuilder(TSharedPtr<IPropertyHandle> InPropertySheetStructHandle, TSharedPtr<IPropertyHandle> InChildPtrToThisContainer, FAudioPropertiesSheet* InLeafMostSheet /*= nullptr*/, bool InIsLeaf /*= false*/)
	: CachedSheetStructHandle(InPropertySheetStructHandle)
	, ChildPtrToThisContainer(InChildPtrToThisContainer)
	, LeafMostSheet(InLeafMostSheet)
	, bIsLeaf(InIsLeaf)
{
	if (CachedSheetStructHandle)
	{
		CachedSheetStructHandle->GetOuterObjects(PropertyOwners);
		ParentPropertyHandle = CachedSheetStructHandle->GetChildHandle(AudioPropertiesSheetDetailsPrivate::ParentPropertyName);
	}
}

void FAudioPropertiesSheetAssetTreeNodeBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{	
	const UObject* ParentAsset = AudioPropertiesSheetDetailsPrivate::GetUObjectFromParentPtrPropertyHandle(ParentPropertyHandle);
	
	NodeRow.NameContent()[BuildHeaderRowName()]
		.ValueContent()[BuildHeaderRowValue(ParentAsset, bIsLeaf)]
		.ShouldAutoExpand(true);		
}

void FAudioPropertiesSheetAssetTreeNodeBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	AddParentNode(ChildrenBuilder);
}

void FAudioPropertiesSheetAssetTreeNodeBuilder::AddParentNode(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (const UObject* ParentAsset = AudioPropertiesSheetDetailsPrivate::GetUObjectFromParentPtrPropertyHandle(ParentPropertyHandle))
	{
		TArray<UObject*> ParentObjects;
		ParentObjects.Add(const_cast<UObject*>(ParentAsset));

		auto ParentObjectsRow = ChildrenBuilder.AddExternalObjectProperty(ParentObjects, TEXT("PropertiesSheet"));
		auto ParentRowPropertyHandle = ParentObjectsRow->GetPropertyHandle();
		ParentObjectsRow->Visibility(EVisibility::Hidden);

		constexpr bool bIsParentLeaf = false;
		const TSharedRef<FAudioPropertiesSheetAssetTreeNodeBuilder> ParentDetails = MakeShareable(new FAudioPropertiesSheetAssetTreeNodeBuilder(ParentRowPropertyHandle, ParentPropertyHandle, LeafMostSheet, bIsParentLeaf));
		ChildrenBuilder.AddCustomBuilder(ParentDetails);
	}
}

FName FAudioPropertiesSheetAssetTreeNodeBuilder::GetName() const
{
	static const FName AudioPropertiesSheetAssetTreeNodeName("AudioPropertiesSheetAssetTreeNode");
	return AudioPropertiesSheetAssetTreeNodeName;
}

TSharedRef<SWidget> FAudioPropertiesSheetAssetTreeNodeBuilder::BuildParentPickerWidget()
{
	FOnGetAllowedClasses AssetPickerAllowedClasses = FOnGetAllowedClasses::CreateLambda([](TArray<const UClass*>& OutClasses) {OutClasses.Add(UAudioPropertiesSheetAsset::StaticClass()); });
	FOnAssetSelected AssetPickerAssetSelected = FOnAssetSelected::CreateSP(this, &FAudioPropertiesSheetAssetTreeNodeBuilder::OnParentAssetSelected);
	FOnShouldFilterAsset AssetPickerFilter = FOnShouldFilterAsset::CreateSP(this, &FAudioPropertiesSheetAssetTreeNodeBuilder::FilterParentAssetForPicking);
	FSimpleDelegate AssetPickerClose = FSimpleDelegate::CreateSP(this, &FAudioPropertiesSheetAssetTreeNodeBuilder::OnAssetPickerClosed);

	return PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
		FAssetData(),
		true,
		true,
		{ UAudioPropertiesSheetAsset::StaticClass() },
		TArray<UFactory*>(),
		AssetPickerFilter,
		AssetPickerAssetSelected,
		AssetPickerClose,
		ParentPropertyHandle);
}

TSharedRef<SWidget> FAudioPropertiesSheetAssetTreeNodeBuilder::BuildHeaderRowName() const
{
	FText HeaderValueText;


	auto GenerateOwningSheetText = [this]()
	{
		if (bIsLeaf || !PropertyOwners[0])
		{
			return FText();
		}
		
		FString OwningAssetName;
		PropertyOwners[0]->GetName(OwningAssetName);
		const FString ParentAssetPropertyString = FString::Printf(TEXT("[%s] "), *OwningAssetName);

		return FText::FromString(ParentAssetPropertyString);
	};

	return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text_Lambda(GenerateOwningSheetText)
					.Font(IDetailLayoutBuilder::GetDetailFontItalic())
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("AudioPropertiesSheetParentRowName", "Parent"))
				];
}

TSharedRef<SWidget> FAudioPropertiesSheetAssetTreeNodeBuilder::BuildHeaderRowValue(const UObject* PropertyOwner, const bool InBuildAssetPicker)
{
	check(CachedSheetStructHandle)

	TSharedPtr<SWidget> AssetRowValue = nullptr;
	FText OwnerNameText = PropertyOwner ?  FText::FromString(PropertyOwner->GetName()) : LOCTEXT("AudioPropertiesSheetNodeDetailsHeaderValue", "None");


	TSharedRef<SWidget> ValueTextWidget = SNew(STextBlock).Text(OwnerNameText)
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
		.ToolTip(CreatePathTooltip(PropertyOwner));


	if (InBuildAssetPicker)
	{
		TSharedRef<SWidget> ComboBoxContent = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.Padding(5, 0)
		[
			ValueTextWidget
		];

		AssetRowValue = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FAudioPropertiesSheetAssetTreeNodeBuilder::BuildParentPickerWidget)
			.ButtonContent()
			[
				ComboBoxContent
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			PropertyCustomizationHelpers::MakeUseSelectedButton(
			FSimpleDelegate::CreateSP(this, &FAudioPropertiesSheetAssetTreeNodeBuilder::OnUseSelected))
		]
		+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeBrowseButton(
				FSimpleDelegate::CreateSP(this, &FAudioPropertiesSheetAssetTreeNodeBuilder::OnBrowseTo, const_cast<UObject*>(PropertyOwner)))
			]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeClearButton(
				FSimpleDelegate::CreateSP(this, &FAudioPropertiesSheetAssetTreeNodeBuilder::OnParentCleared))
			];
	}
	else
	{
		 TSharedRef<SHorizontalBox> AssetRowBox = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				ValueTextWidget
			];

			if (PropertyOwner)
			{
				AssetRowBox->AddSlot().VAlign(VAlign_Center)
				.AutoWidth()
				[
					PropertyCustomizationHelpers::MakeBrowseButton(
					FSimpleDelegate::CreateSP(this, &FAudioPropertiesSheetAssetTreeNodeBuilder::OnBrowseTo, const_cast<UObject*>(PropertyOwner)))
				];				
			}

			AssetRowValue = AssetRowBox;
	}


	return AssetRowValue.ToSharedRef();
}

TSharedRef<SToolTip> FAudioPropertiesSheetAssetTreeNodeBuilder::CreatePathTooltip(const UObject* TargetObject)
{
	FText OwnerTooltip = LOCTEXT("AudioPropertiesSheetNodeDetailsHeaderTooltip", "None");

	if (TargetObject)
	{
		if (TargetObject->GetPackage())
		{
			OwnerTooltip = FText::FromString(TargetObject->GetPackage()->GetPathName());

		}
	}

	return SNew(SToolTip)
		[
			SNew(STextBlock)
			.Text(OwnerTooltip)
		];
}

void FAudioPropertiesSheetAssetTreeNodeBuilder::OnUseSelected()
{
	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);

	if (SelectedAssets.Num() > 0)
	{
		if (!FilterParentAssetForPicking(SelectedAssets[0]))
		{
			OnParentAssetSelected(SelectedAssets[0]);
		}
		else
		{
			UE_LOGF(LogAudioPropertiesEditor, Log, "Trying to select invalid Audio Properties Sheet parent from content browser")
		}
		
	}
}

void FAudioPropertiesSheetAssetTreeNodeBuilder::OnBrowseTo(UObject* AssetToBrowseTo)
{
	TArray<UObject*> AssetsToBrowseTo;
	AssetsToBrowseTo.Add(AssetToBrowseTo);
	GEditor->SyncBrowserToObjects(AssetsToBrowseTo);
}

bool FAudioPropertiesSheetAssetTreeNodeBuilder::FilterParentAssetForPicking(const FAssetData& InAssetData)
{
	if (InAssetData.GetClass() == UAudioPropertiesSheetAsset::StaticClass())
	{
		TArray<UPackage*> OuterPackages;
		ChildPtrToThisContainer->GetOuterPackages(OuterPackages);
		
		for (const UPackage* Package : OuterPackages)
		{
			if (Package->GetOutermost() == InAssetData.GetPackage())
			{
				//Found a matching package with the same asset data, filter it out
				return true;
			}
		}

		if (const UAudioPropertiesSheetAsset* PropertiesSheetAssetPtr = Cast<UAudioPropertiesSheetAsset>(InAssetData.GetAsset()))
		{
			TArray<UObject*> ContainerObjects;
			ChildPtrToThisContainer->GetOuterObjects(ContainerObjects);
			{
				for (const UObject* Object : ContainerObjects)
				{
					if (const UAudioPropertiesSheetAsset* PropertyOwner = Cast<UAudioPropertiesSheetAsset>(Object))
					{
						if (AudioPropertiesUtils::IsSheetInInheritanceTree(*PropertiesSheetAssetPtr, PropertyOwner))
						{
							return true;
						}
					}
				}
			}
		}

		return false;
	}

	return true;
}

void FAudioPropertiesSheetAssetTreeNodeBuilder::OnParentAssetSelected(const FAssetData& AssetData)
{
	ChildPtrToThisContainer->SetValueFromFormattedString(AssetData.IsValid() ? AssetData.GetAsset()->GetPathName() : TEXT("None"));
	ChildPtrToThisContainer->NotifyPostChange(EPropertyChangeType::ValueSet);
	ChildPtrToThisContainer->RequestRebuildChildren();
	OnRebuildChildren.ExecuteIfBound();
}

void FAudioPropertiesSheetAssetTreeNodeBuilder::OnParentCleared()
{
	ChildPtrToThisContainer->SetValueFromFormattedString(TEXT("None"));
	ChildPtrToThisContainer->NotifyPostChange(EPropertyChangeType::ValueSet);
	ChildPtrToThisContainer->RequestRebuildChildren();
	OnRebuildChildren.ExecuteIfBound();
}

void FAudioPropertiesSheetAssetTreeNodeBuilder::OnAssetPickerClosed()
{
	FSlateApplication::Get().DismissAllMenus();
}

#undef LOCTEXT_NAMESPACE