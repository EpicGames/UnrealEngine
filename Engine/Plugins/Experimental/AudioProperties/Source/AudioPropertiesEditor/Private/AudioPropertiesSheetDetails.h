// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomNodeBuilder.h"
#include "IPropertyTypeCustomization.h"

class SToolTip;
class SWidget;
struct FAudioPropertiesSheet;

//Details Customization of an AudioPropertiesSheetAsset
class FAudioPropertiesSheetDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** ~Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	/** ~End IPropertyTypeCustomization interface */
};

//Creates a property node showing the local property bag + any properties inherited from the parent tree
class FAudioPropertiesSheetBagNodeBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FAudioPropertiesSheetBagNodeBuilder>
{
public:
	FAudioPropertiesSheetBagNodeBuilder(
		TSharedPtr<IPropertyHandle> InPropertySheetStructHandle,
		TSharedPtr<IPropertyUtilities> InPropertyUtils, 
		FAudioPropertiesSheet* InLeafMostSheet = nullptr
	);

	void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder )override;
	void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;

	FName GetName() const override;

	void OnParentAssetPropertyOverrideChange(const FProperty& TargetProperty, bool bIsOverridden);


private:
	void AddParentTreePropertyBags(IDetailChildrenBuilder& ChildrenBuilder);

	TSharedRef<SWidget> BuildHeaderRowName() const;
	TSharedRef<SWidget> BuildHeaderRowValue();

	TSharedPtr<IPropertyHandle> CachedSheetStructHandle;
	TSharedPtr<IPropertyHandle> ParentPropertyHandle;

	TSharedPtr<IPropertyUtilities> CachedPropertyUtils;
	FAudioPropertiesSheet* LeafMostSheet;

	TSharedPtr<class IPropertyChangeListener> PropertyChangeListener;

};

//Creates a property node showing the property sheet inheritance tree
class FAudioPropertiesSheetAssetTreeNodeBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FAudioPropertiesSheetAssetTreeNodeBuilder>
{
public:
	FAudioPropertiesSheetAssetTreeNodeBuilder(
		TSharedPtr<IPropertyHandle> InPropertySheetStructHandle, 
		TSharedPtr<IPropertyHandle> InChildPtrToThisContainer,
		FAudioPropertiesSheet* InLeafMostSheet = nullptr,
		bool InIsLeaf = false);

	void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;

	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override { OnRebuildChildren = InOnRegenerateChildren; }

	FName GetName() const override;

	FSimpleDelegate OnRebuildChildren;

private:
	TSharedRef<SWidget> BuildParentPickerWidget();
	TSharedRef<SWidget> BuildHeaderRowName() const;
	TSharedRef<SWidget> BuildHeaderRowValue(const UObject* PropertyOwner, const bool InBuildAssetPicker);
	TSharedRef<SToolTip> CreatePathTooltip(const UObject* TargetObject);

	void AddParentNode(IDetailChildrenBuilder& ChildrenBuilder);

	TSharedPtr<IPropertyHandle> CachedSheetStructHandle;
	TSharedPtr<IPropertyHandle> ParentPropertyHandle;
	TSharedPtr<IPropertyHandle> ChildPtrToThisContainer;

	const IPropertyTypeCustomizationUtils* CustomizationUtils;

	//asset browsing/picking delegates
	bool FilterParentAssetForPicking(const FAssetData& InAssetData);
	void OnParentAssetSelected(const FAssetData& AssetData);
	void OnParentCleared();
	void OnAssetPickerClosed();
	void OnUseSelected();
	void OnBrowseTo(UObject* AssetToBrowseTo);

	TArray<UObject*> PropertyOwners;

	FAudioPropertiesSheet* LeafMostSheet;

	bool bIsLeaf;
};