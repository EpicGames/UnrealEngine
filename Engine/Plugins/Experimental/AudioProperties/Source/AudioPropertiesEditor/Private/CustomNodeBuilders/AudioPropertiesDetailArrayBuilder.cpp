// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesDetailArrayBuilder.h"

#include "AudioPropertiesDetailsInjectorUtils.h"
#include "IDetailChildrenBuilder.h"

FAudioPropertiesDetailArrayBuilder::FAudioPropertiesDetailArrayBuilder(const TSharedRef<IPropertyHandle>& InBaseProperty, const TSharedRef<IPropertyHandle>& InPropertySheetHandle, TWeakObjectPtr<UObject> InPropertyOwner, bool InGenerateHeader, bool InDisplayResetToDefault, bool InDisplayElementNum)
	: FDetailArrayBuilder(InBaseProperty, InGenerateHeader, InDisplayResetToDefault, InDisplayElementNum)
	, PropertySheetPropertyHandle(InPropertySheetHandle)
	, PropertyOwnerObject(InPropertyOwner)
{}

void FAudioPropertiesDetailArrayBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	FDetailArrayBuilder::GenerateHeaderRowContent(NodeRow);
	
	TSharedPtr<IPropertyHandle> ArrayPropertyHandle = GetPropertyHandle();
	TObjectPtr<UObject> PropertyOwner = PropertyOwnerObject.Get();

	const TAttribute<bool> ShouldValueContentBeEnabled = TAttribute<bool>::CreateSP(this, &FAudioPropertiesDetailArrayBuilder::IsEnabled);
	
	NodeRow
	.NameContent()
	[
		AudioPropertiesDetailsInjectorUtils::CreateOverriddenPropertyNameWidget(ArrayPropertyHandle, PropertySheetPropertyHandle, {PropertyOwner})
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		.IsEnabled(ShouldValueContentBeEnabled)
		+ SHorizontalBox::Slot()
		[
			ArrayPropertyHandle->CreatePropertyValueWidget()
		]
	];

	SetUpEditCondition(NodeRow);

	if (GetShouldDisplayResetToDefault())
	{
		TSharedPtr<SResetToDefaultMenu> ResetToDefaultMenu;

		NodeRow.ResetToDefaultContent()
		[
			SNew(SHorizontalBox)
			.IsEnabled(ShouldValueContentBeEnabled)
			+ SHorizontalBox::Slot()
			[
				SAssignNew(ResetToDefaultMenu, SResetToDefaultMenu)
			]
		];

		ResetToDefaultMenu->AddProperty(GetPropertyHandle().ToSharedRef());
	}
}

void FAudioPropertiesDetailArrayBuilder::GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder )
{
	TSharedPtr<IPropertyHandleArray> ArrayPropertyHandle = GetPropertyHandle()->AsArray();
	check(ArrayPropertyHandle);

	TAttribute<bool> ShouldArrayChildrenBeEnabled = TAttribute<bool>::CreateSP(this, &FAudioPropertiesDetailArrayBuilder::IsEnabled);
	uint32 NumChildren = 0;
	ArrayPropertyHandle->GetNumElements( NumChildren );

	for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
	{
		TSharedRef<IPropertyHandle> ElementHandle = ArrayPropertyHandle->GetElement( ChildIndex );

		IDetailPropertyRow& Row = ChildrenBuilder.AddProperty(ElementHandle);
		Row.IsEnabled(ShouldArrayChildrenBeEnabled);
	}
}

bool FAudioPropertiesDetailArrayBuilder::IsEnabled() const
{
	const TSharedPtr<IPropertyHandle> ArrayPropertyHandle = GetPropertyHandle();
	const TObjectPtr<UObject> PropertyOwner = PropertyOwnerObject.Get();
	const bool bEditableProperty = ArrayPropertyHandle->IsEditable();
	const bool bHeaderRowEditConditionValue = GetHeaderRowEditConditionValue();
	const bool bShouldParseProperty = AudioPropertiesDetailsInjectorUtils::ShouldParseProperty(PropertyOwner, ArrayPropertyHandle);

	return bEditableProperty && bHeaderRowEditConditionValue && !bShouldParseProperty;
}

bool FAudioPropertiesDetailArrayBuilder::GetHeaderRowEditConditionValue() const
{
	bool bIsPropertyNegated = false;
	const FBoolProperty* EditCondition = PropertyCustomizationHelpers::GetEditConditionProperty(GetPropertyHandle()->GetProperty(), bIsPropertyNegated);

	if (!EditCondition)
	{
		return true;
	}

	const TObjectPtr<UObject> PropertyOwner = PropertyOwnerObject.Get();
	bool EditConditionValue = PropertyOwner ? EditCondition->GetPropertyValue_InContainer(PropertyOwner) : true;
	return bIsPropertyNegated ? !EditConditionValue : EditConditionValue;
	
}

void FAudioPropertiesDetailArrayBuilder::OnHeaderRowEditConditionValueChanged(bool bValue) const
{
	bool bIsPropertyNegated = false;
	FBoolProperty* EditCondition = PropertyCustomizationHelpers::GetEditConditionProperty(GetPropertyHandle()->GetProperty(), bIsPropertyNegated);

	if (EditCondition)
	{
		const bool ValueToSet = bIsPropertyNegated ? !bValue : bValue;
		EditCondition->SetPropertyValue_InContainer(PropertyOwnerObject.Get(), ValueToSet);
	}
}

void FAudioPropertiesDetailArrayBuilder::SetUpEditCondition(FDetailWidgetRow& NodeRow) const
{
	const TSharedPtr<IPropertyHandle> ArrayPropertyHandle = GetPropertyHandle();
	
	const FString EditCondStr = ArrayPropertyHandle->GetMetaData(AudioPropertiesDetailsInjectorUtils::EditConditionPropertyMetadata);

	if (EditCondStr.IsEmpty())
	{
		return;
	}

	bool bIsPropertyNegated = false;
	const FBoolProperty* EditCondition = PropertyCustomizationHelpers::GetEditConditionProperty(GetPropertyHandle()->GetProperty(), bIsPropertyNegated);

	if (!EditCondition)
	{
		return;
	}
	
	if (!EditCondition->HasMetaData(AudioPropertiesDetailsInjectorUtils::InlineEditConditionTogglePropertyMetadata))
	{
		return;
	}

	const TAttribute<bool> NodeRowEditCondition = TAttribute<bool>::CreateSP(this, &FAudioPropertiesDetailArrayBuilder::GetHeaderRowEditConditionValue);
	const FOnBooleanValueChanged OnEditConditionValueChanged = FOnBooleanValueChanged::CreateSP(this , &FAudioPropertiesDetailArrayBuilder::OnHeaderRowEditConditionValueChanged);
	
	NodeRow.EditCondition(NodeRowEditCondition, OnEditConditionValueChanged);

}