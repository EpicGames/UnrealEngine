// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyCustomizationHelpers.h"

class FAudioPropertiesDetailArrayBuilder : public FDetailArrayBuilder, public TSharedFromThis<FAudioPropertiesDetailArrayBuilder>
{
public:
	FAudioPropertiesDetailArrayBuilder(const TSharedRef<IPropertyHandle>& InBaseProperty, const TSharedRef<IPropertyHandle>& InPropertySheetHandle, TWeakObjectPtr<UObject> InPropertyOwner, bool InGenerateHeader = true, bool InDisplayResetToDefault = true, bool InDisplayElementNum = true);

	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;

private:
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;

	bool IsEnabled() const;
	bool GetHeaderRowEditConditionValue() const;
	void OnHeaderRowEditConditionValueChanged(bool bValue) const;
	void SetUpEditCondition(FDetailWidgetRow& NodeRow) const;
	
	TSharedRef<IPropertyHandle> PropertySheetPropertyHandle;
	TWeakObjectPtr<UObject> PropertyOwnerObject;
};
