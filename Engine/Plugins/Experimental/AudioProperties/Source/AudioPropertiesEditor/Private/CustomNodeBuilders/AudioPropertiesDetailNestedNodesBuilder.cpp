// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesDetailNestedNodesBuilder.h"

#include "AudioPropertiesDetailsInjectorUtils.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "SResetToDefaultMenu.h"

FAudioPropertiesDetailNestedNodesBuilder::FAudioPropertiesDetailNestedNodesBuilder(TSharedRef<IPropertyHandle> InBaseProperty, const TSharedRef<IPropertyHandle>& InPropertySheetHandle, TWeakObjectPtr<UObject> InPropertyOwner, bool InGenerateHeader, bool InDisplayResetToDefault)
	: BasePropertyHandle( InBaseProperty )
	, PropertySheetPropertyHandle(InPropertySheetHandle)
	, PropertyOwnerObject(InPropertyOwner)
	, bGenerateHeader( InGenerateHeader)
	, bDisplayResetToDefault(InDisplayResetToDefault)
{
	uint32 NumChildren = 0;
	BasePropertyHandle->GetNumChildren(NumChildren);
	ensureAlwaysMsgf(NumChildren > 0, TEXT("Customizing property handle with no children"));
	
	BasePropertyHandle->MarkHiddenByCustomization();
}

void FAudioPropertiesDetailNestedNodesBuilder::SetDisplayName(const FText& InDisplayName)
{
	DisplayName = InDisplayName;
}

FName FAudioPropertiesDetailNestedNodesBuilder::GetName() const
{
	return BasePropertyHandle->GetProperty()->GetFName();
}

void FAudioPropertiesDetailNestedNodesBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	if (bGenerateHeader)
	{
		const TAttribute<bool> ShouldValueContentBeEnabled = TAttribute<bool>::CreateSP(this, &FAudioPropertiesDetailNestedNodesBuilder::IsEnabled);
		
		FUIAction CopyAction;
		FUIAction PasteAction;
		BasePropertyHandle->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);
		TObjectPtr<UObject> PropertyOwner = PropertyOwnerObject.Get();

		NodeRow
		.FilterString(!DisplayName.IsEmpty() ? DisplayName : BasePropertyHandle->GetPropertyDisplayName())
		.NameContent()
		[
			AudioPropertiesDetailsInjectorUtils::CreateOverriddenPropertyNameWidget(BasePropertyHandle, PropertySheetPropertyHandle, {PropertyOwner})
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			.IsEnabled(ShouldValueContentBeEnabled)
			+ SHorizontalBox::Slot()
			[
				BasePropertyHandle->CreatePropertyValueWidget()
			]
		]
		.CopyAction(CopyAction)
		.PasteAction(PasteAction);

		if (bDisplayResetToDefault)
		{
			TSharedPtr<SResetToDefaultMenu> ResetToDefaultMenu;

			NodeRow.ResetToDefaultContent()
			[
				SNew(SHorizontalBox)
				.IsEnabled(ShouldValueContentBeEnabled)
				+ SHorizontalBox::Slot()
				[
					SAssignNew(ResetToDefaultMenu, SResetToDefaultMenu)
				]
			];

			ResetToDefaultMenu->AddProperty(BasePropertyHandle);
		}

		SetUpEditCondition(NodeRow);
	}
}

void FAudioPropertiesDetailNestedNodesBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	uint32 NumChildren = 0;
	BasePropertyHandle->GetNumChildren( NumChildren );

	TAttribute<bool> ShouldSetChildrenBeEnabled = TAttribute<bool>::CreateSP(this, &FAudioPropertiesDetailNestedNodesBuilder::IsEnabled);
	
	for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
	{
		TSharedPtr<IPropertyHandle> ElementHandle = BasePropertyHandle->GetChildHandle(ChildIndex);

		if (ElementHandle.IsValid())
		{
			IDetailPropertyRow& Row = ChildrenBuilder.AddProperty(ElementHandle.ToSharedRef());
			Row.IsEnabled(ShouldSetChildrenBeEnabled);
		}
	}
}

TSharedPtr<IPropertyHandle> FAudioPropertiesDetailNestedNodesBuilder::GetPropertyHandle() const
{
	return BasePropertyHandle;
}

bool FAudioPropertiesDetailNestedNodesBuilder::IsEnabled() const
{
	TObjectPtr<UObject> PropertyOwner = PropertyOwnerObject.Get();
	const bool bEditableProperty = BasePropertyHandle->IsEditable();
	const bool bHeaderRowEditConditionValue = GetHeaderRowEditConditionValue();
	const bool bShouldParseProperty = AudioPropertiesDetailsInjectorUtils::ShouldParseProperty(PropertyOwner, BasePropertyHandle);

	return bEditableProperty && bHeaderRowEditConditionValue && !bShouldParseProperty;
}

bool FAudioPropertiesDetailNestedNodesBuilder::GetHeaderRowEditConditionValue() const
{
	bool bIsPropertyNegated = false;
	FBoolProperty* EditCondition = PropertyCustomizationHelpers::GetEditConditionProperty(GetPropertyHandle()->GetProperty(), bIsPropertyNegated);

	if (!EditCondition)
	{
		return true;
	}

	bool EditConditionValue = PropertyOwnerObject.Get() ? EditCondition->GetPropertyValue_InContainer(PropertyOwnerObject.Get()) : true;
	return bIsPropertyNegated ? !EditConditionValue : EditConditionValue;
	
}

void FAudioPropertiesDetailNestedNodesBuilder::OnHeaderRowEditConditionValueChanged(bool bValue) const
{
	bool bIsPropertyNegated = false;
	FBoolProperty* EditCondition = PropertyCustomizationHelpers::GetEditConditionProperty(GetPropertyHandle()->GetProperty(), bIsPropertyNegated);

	if (EditCondition)
	{
		const bool ValueToSet = bIsPropertyNegated ? !bValue : bValue;
		EditCondition->SetPropertyValue_InContainer(PropertyOwnerObject.Get(), ValueToSet);
	}
}

void FAudioPropertiesDetailNestedNodesBuilder::SetUpEditCondition(FDetailWidgetRow& NodeRow) const
{
	TSharedPtr<IPropertyHandle> ArrayPropertyHandle = GetPropertyHandle();
	
	const FString EditCondStr = ArrayPropertyHandle->GetMetaData(AudioPropertiesDetailsInjectorUtils::EditConditionPropertyMetadata);

	if (EditCondStr.IsEmpty())
	{
		return;
	}

	bool bIsPropertyNegated = false;
	FBoolProperty* EditCondition = PropertyCustomizationHelpers::GetEditConditionProperty(GetPropertyHandle()->GetProperty(), bIsPropertyNegated);

	if (!EditCondition)
	{
		return;
	}
	
	if (!EditCondition->HasMetaData(AudioPropertiesDetailsInjectorUtils::InlineEditConditionTogglePropertyMetadata))
	{
		return;
	}

	const TAttribute<bool> NodeRowEditCondition = TAttribute<bool>::CreateSP(this, &FAudioPropertiesDetailNestedNodesBuilder::GetHeaderRowEditConditionValue);
	const FOnBooleanValueChanged OnEditConditionValueChanged = FOnBooleanValueChanged::CreateSP(this , &FAudioPropertiesDetailNestedNodesBuilder::OnHeaderRowEditConditionValueChanged);
	
	NodeRow.EditCondition(NodeRowEditCondition, OnEditConditionValueChanged);
}
