// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomNodeBuilder.h"
#include "PropertyHandle.h"

class FAudioPropertiesDetailNestedNodesBuilder : public IDetailCustomNodeBuilder , public TSharedFromThis<FAudioPropertiesDetailNestedNodesBuilder>
{
public:
	FAudioPropertiesDetailNestedNodesBuilder(TSharedRef<IPropertyHandle> InBaseProperty, const TSharedRef<IPropertyHandle>& InPropertySheetHandle, TWeakObjectPtr<UObject> InPropertyOwner, bool InGenerateHeader = true, bool InDisplayResetToDefault = true);

	virtual bool RequiresTick() const override { return false; }
	virtual void Tick( float DeltaTime ) override {}
	
	void SetDisplayName( const FText& InDisplayName );
	virtual FName GetName() const override;

	virtual bool InitiallyCollapsed() const override { return false; }

	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override;

private:
	bool IsEnabled() const;
	bool GetHeaderRowEditConditionValue() const;
	void OnHeaderRowEditConditionValueChanged(bool bValue) const;
	void SetUpEditCondition(FDetailWidgetRow& NodeRow) const;
	
	FText DisplayName;
	TSharedRef<IPropertyHandle> BasePropertyHandle;
	TSharedRef<IPropertyHandle> PropertySheetPropertyHandle;
	TWeakObjectPtr<UObject> PropertyOwnerObject;

	bool bGenerateHeader;
	bool bDisplayResetToDefault;
	
};
