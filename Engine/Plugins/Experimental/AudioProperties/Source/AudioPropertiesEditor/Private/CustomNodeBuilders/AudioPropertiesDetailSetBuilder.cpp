// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioPropertiesDetailSetBuilder.h"
#include "AudioPropertiesDetailsInjectorUtils.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "SResetToDefaultMenu.h"
	
FAudioPropertiesDetailSetBuilder::FAudioPropertiesDetailSetBuilder(TSharedRef<IPropertyHandle> InBaseProperty, const TSharedRef<IPropertyHandle>& InPropertySheetHandle, TWeakObjectPtr<UObject> InPropertyOwner, bool InGenerateHeader, bool InDisplayResetToDefault, bool InDisplayElementNum)
	: SetPropertyHandle( InBaseProperty->AsSet() )
	, BasePropertyHandle( InBaseProperty )
	, PropertySheetPropertyHandle(InPropertySheetHandle)
	, PropertyOwnerObject(InPropertyOwner)
	, bGenerateHeader( InGenerateHeader)
	, bDisplayResetToDefault(InDisplayResetToDefault)
	, bDisplayElementNum(InDisplayElementNum)
{
	check( SetPropertyHandle.IsValid() );

	FSimpleDelegate OnNumChildrenChanged = FSimpleDelegate::CreateRaw( this, &FAudioPropertiesDetailSetBuilder::OnNumChildrenChanged );
	OnNumElementsChangedHandle = SetPropertyHandle->SetOnNumElementsChanged( OnNumChildrenChanged );
	
	BasePropertyHandle->MarkHiddenByCustomization();
}

FAudioPropertiesDetailSetBuilder::~FAudioPropertiesDetailSetBuilder()
{
	SetPropertyHandle->UnregisterOnNumElementsChanged(OnNumElementsChangedHandle);
}

void FAudioPropertiesDetailSetBuilder::SetDisplayName(const FText& InDisplayName)
{
	DisplayName = InDisplayName;
}

FName FAudioPropertiesDetailSetBuilder::GetName() const
{
	return BasePropertyHandle->GetProperty()->GetFName();
}

void FAudioPropertiesDetailSetBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	if (bGenerateHeader)
	{
		const TAttribute<bool> ShouldValueContentBeEnabled = TAttribute<bool>::CreateSP<>(this, &FAudioPropertiesDetailSetBuilder::IsEnabled);

		TSharedPtr<SHorizontalBox> ContentHorizontalBox;
		SAssignNew(ContentHorizontalBox, SHorizontalBox);
		if (bDisplayElementNum)
		{
			ContentHorizontalBox->AddSlot()
			[
				BasePropertyHandle->CreatePropertyValueWidget()
			];
		}

		FUIAction CopyAction;
		FUIAction PasteAction;
		BasePropertyHandle->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);
		TObjectPtr<UObject> PropertyOwner = PropertyOwnerObject.Get();

		NodeRow
		.FilterString(!DisplayName.IsEmpty() ? DisplayName : BasePropertyHandle->GetPropertyDisplayName())
		.NameContent()
		[
			AudioPropertiesDetailsInjectorUtils::CreateOverriddenPropertyNameWidget(BasePropertyHandle, PropertySheetPropertyHandle, {PropertyOwner})
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			.IsEnabled(ShouldValueContentBeEnabled)
			+ SHorizontalBox::Slot()
			[
				BasePropertyHandle->CreatePropertyValueWidget()
			]
		]
		.CopyAction(CopyAction)
		.PasteAction(PasteAction);

		if (bDisplayResetToDefault)
		{
			TSharedPtr<SResetToDefaultMenu> ResetToDefaultMenu;

			NodeRow.ResetToDefaultContent()
			[
				SNew(SHorizontalBox)
				.IsEnabled(ShouldValueContentBeEnabled)
				+ SHorizontalBox::Slot()
				[
					SAssignNew(ResetToDefaultMenu, SResetToDefaultMenu)
				]
			];

			ResetToDefaultMenu->AddProperty(BasePropertyHandle);
		}

		SetUpEditCondition(NodeRow);
	}
}

void FAudioPropertiesDetailSetBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	uint32 NumChildren = 0;
	SetPropertyHandle->GetNumElements( NumChildren );

	TAttribute<bool> ShouldSetChildrenBeEnabled = TAttribute<bool>::CreateSP(this, &FAudioPropertiesDetailSetBuilder::IsEnabled);
	
	for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
	{
		TSharedRef<IPropertyHandle> ElementHandle = SetPropertyHandle->GetElement( ChildIndex );
		IDetailPropertyRow& Row = ChildrenBuilder.AddProperty(ElementHandle);
		Row.IsEnabled(ShouldSetChildrenBeEnabled);
	}
}

TSharedPtr<IPropertyHandle> FAudioPropertiesDetailSetBuilder::GetPropertyHandle() const
{
	return BasePropertyHandle;
}

void FAudioPropertiesDetailSetBuilder::OnNumChildrenChanged() const
{
	OnRebuildChildren.ExecuteIfBound();
}

bool FAudioPropertiesDetailSetBuilder::IsEnabled() const
{
	TObjectPtr<UObject> PropertyOwner = PropertyOwnerObject.Get();
	const bool bEditableProperty = BasePropertyHandle->IsEditable();
	const bool bHeaderRowEditConditionValue = GetHeaderRowEditConditionValue();
	const bool bShouldParseProperty = AudioPropertiesDetailsInjectorUtils::ShouldParseProperty(PropertyOwner, BasePropertyHandle);

	return bEditableProperty && bHeaderRowEditConditionValue && !bShouldParseProperty;
}

bool FAudioPropertiesDetailSetBuilder::GetHeaderRowEditConditionValue() const
{
	bool bIsPropertyNegated = false;
	FBoolProperty* EditCondition = PropertyCustomizationHelpers::GetEditConditionProperty(GetPropertyHandle()->GetProperty(), bIsPropertyNegated);

	if (!EditCondition)
	{
		return true;
	}

	bool EditConditionValue = PropertyOwnerObject.Get() ? EditCondition->GetPropertyValue_InContainer(PropertyOwnerObject.Get()) : true;
	return bIsPropertyNegated ? !EditConditionValue : EditConditionValue;
	
}

void FAudioPropertiesDetailSetBuilder::OnHeaderRowEditConditionValueChanged(bool bValue) const
{
	bool bIsPropertyNegated = false;
	FBoolProperty* EditCondition = PropertyCustomizationHelpers::GetEditConditionProperty(GetPropertyHandle()->GetProperty(), bIsPropertyNegated);

	if (EditCondition)
	{
		const bool ValueToSet = bIsPropertyNegated ? !bValue : bValue;
		EditCondition->SetPropertyValue_InContainer(PropertyOwnerObject.Get(), ValueToSet);
	}
}

void FAudioPropertiesDetailSetBuilder::SetUpEditCondition(FDetailWidgetRow& NodeRow) const
{
	TSharedPtr<IPropertyHandle> ArrayPropertyHandle = GetPropertyHandle();
	
	const FString EditCondStr = ArrayPropertyHandle->GetMetaData(AudioPropertiesDetailsInjectorUtils::EditConditionPropertyMetadata);

	if (EditCondStr.IsEmpty())
	{
		return;
	}

	bool bIsPropertyNegated = false;
	FBoolProperty* EditCondition = PropertyCustomizationHelpers::GetEditConditionProperty(GetPropertyHandle()->GetProperty(), bIsPropertyNegated);

	if (!EditCondition)
	{
		return;
	}
	
	if (!EditCondition->HasMetaData(AudioPropertiesDetailsInjectorUtils::InlineEditConditionTogglePropertyMetadata))
	{
		return;
	}

	const TAttribute<bool> NodeRowEditCondition = TAttribute<bool>::CreateSP(this, &FAudioPropertiesDetailSetBuilder::GetHeaderRowEditConditionValue);
	const FOnBooleanValueChanged OnEditConditionValueChanged = FOnBooleanValueChanged::CreateSP(this , &FAudioPropertiesDetailSetBuilder::OnHeaderRowEditConditionValueChanged);
	
	NodeRow.EditCondition(NodeRowEditCondition, OnEditConditionValueChanged);
}