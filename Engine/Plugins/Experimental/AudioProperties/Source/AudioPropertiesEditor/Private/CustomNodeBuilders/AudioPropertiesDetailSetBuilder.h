// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomNodeBuilder.h"
#include "PropertyHandle.h"

class FAudioPropertiesDetailSetBuilder : public IDetailCustomNodeBuilder , public TSharedFromThis<FAudioPropertiesDetailSetBuilder>
{
public:
	FAudioPropertiesDetailSetBuilder(TSharedRef<IPropertyHandle> InBaseProperty, const TSharedRef<IPropertyHandle>& InPropertySheetHandle, TWeakObjectPtr<UObject> InPropertyOwner, bool InGenerateHeader = true, bool InDisplayResetToDefault = true, bool InDisplayElementNum = true);
	virtual ~FAudioPropertiesDetailSetBuilder();

	virtual bool RequiresTick() const override { return false; }
	virtual void Tick( float DeltaTime ) override {}

	void SetDisplayName( const FText& InDisplayName );
	virtual FName GetName() const override;

	virtual bool InitiallyCollapsed() const override { return false; }

	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override;

protected:
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRebuildChildren) override { OnRebuildChildren = InOnRebuildChildren; }
	void OnNumChildrenChanged() const;

private:
	bool IsEnabled() const;
	bool GetHeaderRowEditConditionValue() const;
	void OnHeaderRowEditConditionValueChanged(bool bValue) const;
	void SetUpEditCondition(FDetailWidgetRow& NodeRow) const;
	
	FText DisplayName;
	TSharedPtr<IPropertyHandleSet> SetPropertyHandle;
	TSharedRef<IPropertyHandle> BasePropertyHandle;
	TSharedRef<IPropertyHandle> PropertySheetPropertyHandle;
	TWeakObjectPtr<UObject> PropertyOwnerObject;
	
	FSimpleDelegate OnRebuildChildren;
	FDelegateHandle OnNumElementsChangedHandle;
	
	bool bGenerateHeader;
	bool bDisplayResetToDefault;
	bool bDisplayElementNum;
};
