// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedAudioPropertyBagDetails.h"

#include "AudioPropertiesSheet.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "PropertyBagDetails.h"
#include "ScopedTransaction.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "InstancedAudioPropertyBagDetails.h"

namespace InstancedAudioPropertyBagDetailsCVars
{
	static bool ShowIDsOnAudioPropertyBagDetailsCVar = 0;
	FAutoConsoleVariableRef CVarAudioPropertyBagDetailsShowIDs(
		TEXT("au.AudioProperties.ShowPropertiesIDs"),
		ShowIDsOnAudioPropertyBagDetailsCVar,
		TEXT("Show properties IDs on audio property sheet details.\n")
		TEXT("0: Disabled (default), 1: Enabled"),
		ECVF_Default);
}

namespace InstancedAudioPropertyBagDetailsPrivate
{
	FGuid GetPropertyID(const FAudioPropertiesSheet& InLeafMostPropertySheet, TSharedPtr<IPropertyHandle> InProperty)
	{
		if (!InProperty)
		{
			return FGuid();
		}

		auto FindPropertyID = [](const FAudioPropertiesSheet& PropertySheet, const FName& PropertyName) -> FGuid
		{
			if (const FPropertyBagPropertyDesc* PropertyDesc = PropertySheet.Properties.FindPropertyDescByName(PropertyName))
			{
				return PropertyDesc->ID;
			}
			return FGuid(); 
		};

		const FName PropertyName = InProperty->GetProperty()->GetFName();

		// Find the closest parent sheet and check for the property ID
		if (TObjectPtr<const UAudioPropertiesSheetAsset> OwningSheet = InLeafMostPropertySheet.FindClosestParentWithProperty(*InProperty->GetProperty()).Value)
		{
			return InLeafMostPropertySheet.IsPropertyOverridden(*InProperty->GetProperty()) ? FindPropertyID(InLeafMostPropertySheet, PropertyName) : FindPropertyID(OwningSheet->PropertiesSheet, PropertyName);
		}

		//in case we did not find a parent, this property is likely to be local
		return FindPropertyID(InLeafMostPropertySheet, PropertyName);
	}
}

FInstancedAudioPropertyBagDetails::FInstancedAudioPropertyBagDetails(TSharedPtr<IPropertyHandle> InInstancedPropertyBagHandle, const TSharedPtr<IPropertyUtilities>& InPropUtils, const UAudioPropertiesSheetAsset* InOwningAudioPropertiesSheetAsset, FAudioPropertiesSheet* InLeafMostPropertySheet, bool bInIsLeafBag /*= false*/)
 	: FPropertyBagInstanceDataDetails(InInstancedPropertyBagHandle, InPropUtils, /*bInFixedLayout*/ !bInIsLeafBag, /*bInAllowArrays*/ true)
 	, InstancedPropertyBagHandle(InInstancedPropertyBagHandle)
    , OwningAudioPropertiesSheetAsset(InOwningAudioPropertiesSheetAsset)
    , LeafMostPropertySheet(InLeafMostPropertySheet)
	, CachedPropUtils(InPropUtils)
 	, bIsLeafBag(bInIsLeafBag)
{
	check(InstancedPropertyBagHandle.IsValid())
	check(LeafMostPropertySheet)
}

void FInstancedAudioPropertyBagDetails::OnChildRowAdded(IDetailPropertyRow& ChildRow)
{ 
	if (bIsLeafBag)
 	{
		DrawLeafBagChildRow(ChildRow);
		return;
 	}

	DrawInheritedBagChildRow(ChildRow);
}

void FInstancedAudioPropertyBagDetails::DrawLeafBagChildRow(IDetailPropertyRow& ChildRow)
{
	TSharedPtr<IPropertyHandle> ChildPropertyHandle = ChildRow.GetPropertyHandle();
	check(ChildPropertyHandle->GetProperty())

	FPropertyBagInstanceDataDetails::OnChildRowAdded(ChildRow);

	TSharedRef<SWidget> PropertyBagNameWidget = ChildRow.CustomNameWidget()->Widget;
	TSharedRef<SWidget> PropertyBagValueWidget = ChildRow.CustomValueWidget()->Widget;

	ChildRow.CustomWidget(true)
		.NameContent()
		[
			PropertyBagNameWidget
		]
		.ValueContent()
		[
			PropertyBagValueWidget
		]
		.ExtensionContent()
		[
			SNew(STextBlock).Text_Lambda([this, ChildPropertyHandle]()
				{
					return FText::FromString(InstancedAudioPropertyBagDetailsPrivate::GetPropertyID(*LeafMostPropertySheet, ChildPropertyHandle).ToString());
				})
			.Visibility(InstancedAudioPropertyBagDetailsCVars::ShowIDsOnAudioPropertyBagDetailsCVar ? EVisibility::Visible : EVisibility::Hidden)
		];

	TAttribute<bool> bIsInheritedProperty = LeafMostPropertySheet->IsPropertyOverridden(*ChildPropertyHandle->GetProperty());

	if (bIsInheritedProperty.Get() == true)
	{
		FOnBooleanValueChanged OnOverrideChanged = FOnBooleanValueChanged::CreateLambda([this, ChildPropertyHandle](const bool bOverrideValue)
		{
			HandlePropertyOverrideChanged(ChildPropertyHandle, bOverrideValue);
		});
	
		ChildRow.EditCondition(bIsInheritedProperty, OnOverrideChanged);
	}
}

void FInstancedAudioPropertyBagDetails::DrawInheritedBagChildRow(IDetailPropertyRow& ChildRow)
{
	TSharedPtr<IPropertyHandle> ChildPropertyHandle = ChildRow.GetPropertyHandle();
	
	check(ChildPropertyHandle->GetProperty())
	check(InstancedPropertyBagHandle.IsValid());
	check(OwningAudioPropertiesSheetAsset);
	check(LeafMostPropertySheet);

	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;

	ChildRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);
 
	const bool bIsPropertyOverridenByLeafMost = LeafMostPropertySheet->IsPropertyOverridden(*ChildPropertyHandle->GetProperty());
	const bool bIsPropertyOverriddenByChild = LeafMostPropertySheet->FindClosestParentWithProperty(*ChildPropertyHandle->GetProperty()).Value != OwningAudioPropertiesSheetAsset;

	auto GenerateOwningSheetText = [this]()
	{
		if (!OwningAudioPropertiesSheetAsset)
		{
			return FText();
		}

		FString OwningAssetName;
		OwningAudioPropertiesSheetAsset->GetName(OwningAssetName);
		const FString OwningAssetString = FString::Printf(TEXT("[%s] "), *OwningAssetName);

		return FText::FromString(OwningAssetString);
	};

	ChildRow
		.CustomWidget(/*bShowChildren*/true)
		.NameContent()
		[
			SNew(SHorizontalBox)
			// Name
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock).Text_Lambda(GenerateOwningSheetText)
					.Font(IDetailLayoutBuilder::GetDetailFontItalic())
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(FText::FromName(ChildPropertyHandle->GetProperty()->GetFName()))
				]
			]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				ValueWidget.ToSharedRef()
			]
		]
		.ExtensionContent()
		[
			SNew(STextBlock).Text_Lambda([this, ChildPropertyHandle]()
			{
				return FText::FromString(InstancedAudioPropertyBagDetailsPrivate::GetPropertyID(*LeafMostPropertySheet, ChildPropertyHandle).ToString());
			})
			.Visibility(InstancedAudioPropertyBagDetailsCVars::ShowIDsOnAudioPropertyBagDetailsCVar ? EVisibility::Visible : EVisibility::Hidden)
		];

	TAttribute<bool> bIsPropertyOverridden = bIsPropertyOverridenByLeafMost || bIsPropertyOverriddenByChild;
	ChildRow.Visibility(bIsPropertyOverridden.Get() ? EVisibility::Hidden : EVisibility::Visible);

	FOnBooleanValueChanged OnOverrideChanged = FOnBooleanValueChanged::CreateLambda([this, ChildPropertyHandle](const bool bOverrideValue)
	{
		HandlePropertyOverrideChanged(ChildPropertyHandle, bOverrideValue);
	});

	ChildRow.EditCondition(bIsPropertyOverridden, OnOverrideChanged);
}

void FInstancedAudioPropertyBagDetails::HandlePropertyOverrideChanged(TSharedPtr<IPropertyHandle> PropertyThatChanged, const bool bOverrideState)
{
	FText TransactionText = FText::Format(LOCTEXT("OnCheckStateChanged", "Changed Inheritance for property {0} to {1}"),
		FText::FromName(PropertyThatChanged->GetProperty()->GetFName()),
		bOverrideState ? FText::FromString(TEXT("Overridden")) : FText::FromString(TEXT("Inherited")));

	FScopedTransaction Transaction(TransactionText);

	InstancedPropertyBagHandle->NotifyPreChange();
	LeafMostPropertySheet->UpdatePropertyOverride(*PropertyThatChanged->GetProperty(), bOverrideState);
	InstancedPropertyBagHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	InstancedPropertyBagHandle->NotifyFinishedChangingProperties();
	InstancedPropertyBagHandle->RequestRebuildChildren();

	if (CachedPropUtils)
	{
		CachedPropUtils->ForceRefresh();
	}
}

#undef LOCTEXT_NAMESPACE