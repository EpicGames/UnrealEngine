// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAudioPropertiesSheetBuilderWidget.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AudioPropertiesSheet.h"
#include "AudioPropertiesUtils.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/MessageDialog.h"
#include "PropertyCustomizationHelpers.h"
#include "Sound/SoundBase.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableViewBase.h"

#define LOCTEXT_NAMESPACE "SAudioPropertiesSheetBuilderWidget"

namespace AudioPropertiesSheetBuilderWidget
{
	static const FName AddPropertyColumnName = "AddProperty";
	static const FName CopyPropertyValueColumnName = "CopyPropertyValue";
	static const FName PropertyNameColumnName = "PropertyName";
	static const FName PropertyValueColumnName = "PropertyValue";

	const float ColumnItemBoxPadding = 1.f;
	const FVector2D HeaderItemPadding = FVector2D(5.f, 3.f);
	const FVector2D ComboBoxPadding = FVector2D(5.f, 0.f);

	struct FAssetPickerParams
	{
		FAssetData InitialObject = FAssetData();
		bool bAllowClear = false;
		TArray<const UClass*> AllowedClasses;
		FOnAssetSelected OnAssetSelected;
	};

	TSharedRef<SWidget> BuildAssetPickerWidget(const FAssetPickerParams& InAssetPickerParams)
	{
		return PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
			InAssetPickerParams.InitialObject,
			InAssetPickerParams.bAllowClear,
			InAssetPickerParams.AllowedClasses,
			PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(InAssetPickerParams.AllowedClasses),
			FOnShouldFilterAsset(),
			InAssetPickerParams.OnAssetSelected,
			FSimpleDelegate());
	}
		
	class SPropertyRow : public SMultiColumnTableRow<AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr>
	{
	public:
		SLATE_BEGIN_ARGS(SPropertyRow) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr InEntryItem, const UObject* InSourceObjectPtr)
		{
			Item = InEntryItem;
			SourceObjectPtr = InSourceObjectPtr;
			SMultiColumnTableRow<AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (ColumnName == AddPropertyColumnName)
			{
				return  SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(ColumnItemBoxPadding)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([this]()
						{
							return Item->bInheritProperty ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
						.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
						{
							Item->bInheritProperty = NewState == ECheckBoxState::Checked;
							if (!Item->bInheritProperty)
							{
								//if we are not inheriting the property we should not inherit the value
								Item->bInheritValue = false;
							}
						})
					];
        
			}
			else if (ColumnName == CopyPropertyValueColumnName)
			{
					return  SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(ColumnItemBoxPadding)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([this]()
						{
							return Item->bInheritValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
						.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
						{
							Item->bInheritValue = NewState == ECheckBoxState::Checked;
							if (Item->bInheritValue)
							{
								//if we are inheriting the value we also want to inherit the property
								Item->bInheritProperty = true;
							}
						})
					];
			}
			else if (ColumnName == PropertyNameColumnName)
			{
				const FText PropertyName = Item->CachedProperty ? Item->CachedProperty->GetDisplayNameText() : FText();

				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4.f, ColumnItemBoxPadding)
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.Text(PropertyName)
					];
			}
			else if (ColumnName == PropertyValueColumnName)
			{
				FString PropertyValue;
								
				if (SourceObjectPtr && SourceObjectPtr->IsValidLowLevelFast())
				{
					if (const FProperty* CachedProperty = Item->CachedProperty)
					{
						const UAudioPropertiesSheetAsset* SourceAsPropertySheet = Cast<UAudioPropertiesSheetAsset>(SourceObjectPtr);
						const void* PropertyValuePtr = nullptr;
						
						if (SourceAsPropertySheet)
						{
							auto OnLeafPropertyVisited = [&CachedProperty, &PropertyValuePtr](const FPropertyBagPropertyDesc& PropertyDesc, const FAudioPropertiesSheet& OwningSheet)
							{
								if (PropertyDesc.Name == CachedProperty->GetName())
								{
									PropertyValuePtr = CachedProperty->ContainerPtrToValuePtr<void>(OwningSheet.Properties.GetValue().GetMemory());
								}
							};
								
							AudioPropertiesUtils::VisitLeafMostProperties(*SourceAsPropertySheet, OnLeafPropertyVisited);
								
						}
						else
						{
							PropertyValuePtr = CachedProperty->ContainerPtrToValuePtr<void>(SourceObjectPtr);
						}
						
						if(PropertyValuePtr)
						{
							CachedProperty->ExportTextItem_Direct(
								PropertyValue,
								PropertyValuePtr,   
								nullptr,
								nullptr,
								PPF_None			
							);		
						}
					}
				}
				
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4.f, ColumnItemBoxPadding)
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.Text(FText::FromString(PropertyValue))
						.ToolTipText(FText::FromString(PropertyValue))
					];
			}

			return SNullWidget::NullWidget;
		}

	private:
		AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr Item;
		const UObject* SourceObjectPtr = nullptr;
	};
}

void SAudioPropertiesSheetBuilderWidget::Construct(const FArguments& InArgs)
{
    SetSourceObject(InArgs._SourceObject);

	if (InArgs._bSourceIsParent && SourceObject)
	{
		if (ensureAlwaysMsgf(bSourceIsPropertySheet, TEXT("Source should be property sheet to be parent")))
		{
			SetParentAsset(GetSourceAssetData());
		}
	}

    ChildSlot
    [
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			GenerateSourceObjectWidget()
		]
    	+ SVerticalBox::Slot()
		.AutoHeight()
		[
			GenerateParentPickerWidget()
		]
		+ SVerticalBox::Slot()
		[
			GeneratePropertyListWidget()
		]
		+ SVerticalBox::Slot()
        .Padding(10.0f)
		.AutoHeight()
        [
            SNew(SButton)
            .Text(FText::FromString("Make Property Sheet"))
			.HAlign(EHorizontalAlignment::HAlign_Center)
            .OnClicked(this, &SAudioPropertiesSheetBuilderWidget::OnMakePropertySheetClicked)
        ]
    ];
}

TSharedRef<SWidget> SAudioPropertiesSheetBuilderWidget::GenerateSourceObjectWidget()
{
	TSharedRef<SWidget> ComboBoxContent = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.FillWidth(1.0f)
	.VAlign(VAlign_Center)
	.Padding(5, 0)
	[
		SAssignNew(SourcePickerComboText, STextBlock)
		.Text(FText::FromString(SourceObject.GetName()))
	];

	AudioPropertiesSheetBuilderWidget::FAssetPickerParams AssetPickerParams;
	AssetPickerParams.InitialObject = GeneratedSheetParent;
	AssetPickerParams.AllowedClasses = {UAudioPropertiesSheetAsset::StaticClass(), USoundBase::StaticClass()};
	AssetPickerParams.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SAudioPropertiesSheetBuilderWidget::SetSourceAsset);

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(AudioPropertiesSheetBuilderWidget::HeaderItemPadding)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString("Source Object"))
				.Font(FCoreStyle::Get().GetFontStyle("BoldFont"))
				.ToolTipText(FText::FromString("The object we are inheriting properties from"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(AudioPropertiesSheetBuilderWidget::ComboBoxPadding)
			.AutoWidth()
			[
				SAssignNew(SourcePickerComboButton, SComboButton)
				.OnGetMenuContent_Lambda([AssetPickerParams](){return AudioPropertiesSheetBuilderWidget::BuildAssetPickerWidget(AssetPickerParams);})
				.ButtonContent()
				[
					ComboBoxContent
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.f)
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeUseSelectedButton(
					FSimpleDelegate::CreateSP(this, &SAudioPropertiesSheetBuilderWidget::OnSourceUseSelectedButtonClicked)
				)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.f)
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeBrowseButton(
					FSimpleDelegate::CreateSP(this, &SAudioPropertiesSheetBuilderWidget::OnSourceBrowseToButtonClicked)
				)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f)
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeEditButton(
					FSimpleDelegate::CreateSP(this, &SAudioPropertiesSheetBuilderWidget::OnSourceEditButtonClicked)
				)
			]
		];
}

TSharedRef<SWidget> SAudioPropertiesSheetBuilderWidget::GenerateParentPickerWidget()
{
	TSharedRef<SWidget> ComboBoxContent = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.FillWidth(1.0f)
	.VAlign(VAlign_Center)
	.Padding(5, 0)
	[
		SAssignNew(ParentPickerComboText, STextBlock)
		.Text(FText::FromString(GeneratedSheetParent.IsValid() ? GeneratedSheetParent.AssetName.ToString() : "None"))
	];

	AudioPropertiesSheetBuilderWidget::FAssetPickerParams AssetPickerParams;
	AssetPickerParams.InitialObject = GeneratedSheetParent;
	AssetPickerParams.bAllowClear = true;
	AssetPickerParams.AllowedClasses = {UAudioPropertiesSheetAsset::StaticClass()};
	AssetPickerParams.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SAudioPropertiesSheetBuilderWidget::SetParentAsset);

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(AudioPropertiesSheetBuilderWidget::HeaderItemPadding)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString("Generated Sheet Parent"))
				.ToolTipText(FText::FromString("The parent that will be assigned to the generated property sheet"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(AudioPropertiesSheetBuilderWidget::ComboBoxPadding)
			[
				SAssignNew(ParentPickerComboButton, SComboButton)
				.OnGetMenuContent_Lambda([AssetPickerParams](){return AudioPropertiesSheetBuilderWidget::BuildAssetPickerWidget(AssetPickerParams);})
				.ButtonContent()
				[
					ComboBoxContent
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.f)
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeUseSelectedButton(
					FSimpleDelegate::CreateSP(this, &SAudioPropertiesSheetBuilderWidget::OnParentUseSelectedButtonClicked)
				)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.f)
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeBrowseButton(
					FSimpleDelegate::CreateSP(this, &SAudioPropertiesSheetBuilderWidget::OnParentBrowseToButtonClicked)
				)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f)
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeEditButton(
					FSimpleDelegate::CreateSP(this, &SAudioPropertiesSheetBuilderWidget::OnParentEditButtonClicked)
				)
			]
		];
}

void SAudioPropertiesSheetBuilderWidget::UpdatePropertyRequests()
{
	PropertyRequests.Empty();

	UClass* SourceClass = SourceObject->GetClass();
	
	if (SourceClass && !bSourceIsPropertySheet)
	{
		for (TFieldIterator<FProperty> PropIt(SourceClass); PropIt; ++PropIt)
		{
			FProperty* Property = *PropIt;

			if (!Property)
			{
				continue;
			}

			if (Property->HasAllPropertyFlags(CPF_Edit) && !Property->HasAnyPropertyFlags(CPF_EditConst))
			{
				PropertyRequests.Add(MakeShared<AudioPropertiesSheetAssetBuilder::FPropertyRequest>(Property));
			}
		}
	}
	else if (SourceObject && bSourceIsPropertySheet)
	{
		const UAudioPropertiesSheetAsset* SourcePropertySheet = Cast<UAudioPropertiesSheetAsset>(SourceObject.Get());

		if (SourcePropertySheet)
		{
			auto OnLeafPropertyVisited = [this](const FPropertyBagPropertyDesc& PropertyDesc, const FAudioPropertiesSheet& OwningSheet)
				{
					PropertyRequests.Add(MakeShared<AudioPropertiesSheetAssetBuilder::FPropertyRequest>(PropertyDesc.CachedProperty));
				};
			
			AudioPropertiesUtils::VisitLeafMostProperties(*SourcePropertySheet, OnLeafPropertyVisited);
		}
		
	}

	FilteredPropertyRequests = PropertyRequests;

	if (PropertyRequestsView.IsValid())
	{
		PropertyRequestsView->RequestListRefresh();
	}
}

TSharedRef<SWidget> SAudioPropertiesSheetBuilderWidget::GeneratePropertyListWidget()
{
	PropertyRequestsView = SNew(SListView<AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr>)
		.ListItemsSource(&FilteredPropertyRequests)
		.OnGenerateRow(this, &SAudioPropertiesSheetBuilderWidget::OnGenerateRowForList)
		.HeaderRow
		(
			SNew(SHeaderRow)
			+ SHeaderRow::Column(AudioPropertiesSheetBuilderWidget::AddPropertyColumnName)
				.DefaultLabel(LOCTEXT("AddPropertyColumnHeaderName", "Add Property"))
				.HAlignHeader(HAlign_Center)
				.HAlignCell(HAlign_Center)
				.ManualWidth(100)
				.SortMode(this, &SAudioPropertiesSheetBuilderWidget::GetColumnSortMode, AudioPropertiesSheetBuilderWidget::AddPropertyColumnName)
				.OnSort(this, &SAudioPropertiesSheetBuilderWidget::OnColumnSortModeChanged)
				.InitialSortMode(EColumnSortMode::Type::Ascending)
			+ SHeaderRow::Column(AudioPropertiesSheetBuilderWidget::CopyPropertyValueColumnName)
				.DefaultLabel(LOCTEXT("CopyPropertyValueColumnHeaderName", "Copy Property Value"))
				.HAlignHeader(HAlign_Center)
				.HAlignCell(HAlign_Center)
				.ManualWidth(200)
				.SortMode(this, &SAudioPropertiesSheetBuilderWidget::GetColumnSortMode, AudioPropertiesSheetBuilderWidget::CopyPropertyValueColumnName)
				.OnSort(this, &SAudioPropertiesSheetBuilderWidget::OnColumnSortModeChanged)
			+ SHeaderRow::Column(AudioPropertiesSheetBuilderWidget::PropertyNameColumnName)
				.DefaultLabel(LOCTEXT("PropertyNameColumnHeaderName", "Property Name"))
				.SortMode(this, &SAudioPropertiesSheetBuilderWidget::GetColumnSortMode, AudioPropertiesSheetBuilderWidget::PropertyNameColumnName)
				.OnSort(this, &SAudioPropertiesSheetBuilderWidget::OnColumnSortModeChanged)
			+ SHeaderRow::Column(AudioPropertiesSheetBuilderWidget::PropertyValueColumnName)
				.DefaultLabel(LOCTEXT("PropertyValueColumnHeaderName", "Property Value"))
		);

    return SNew(SVerticalBox)
		    + SVerticalBox::Slot()
            .AutoHeight()
			.Padding(AudioPropertiesSheetBuilderWidget::HeaderItemPadding)
            [
                SNew(SEditableTextBox)
                .OnTextChanged(this, &SAudioPropertiesSheetBuilderWidget::OnSearchTextChanged)
				.HintText(FText::FromString("Filter Properties"))
            ]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(AudioPropertiesSheetBuilderWidget::HeaderItemPadding)
			+ SVerticalBox::Slot()
			[
				PropertyRequestsView.ToSharedRef()
			];
}

void SAudioPropertiesSheetBuilderWidget::UpdateParentPickerComboText() const
{
	if (ParentPickerComboText)
	{
		ParentPickerComboText->SetText(FText::FromString(GeneratedSheetParent.IsValid() ? GeneratedSheetParent.AssetName.ToString() : "None" ));
	}
}

void SAudioPropertiesSheetBuilderWidget::UpdateSourcePickerComboText() const
{
	if (SourcePickerComboText)
	{
		SourcePickerComboText->SetText(FText::FromString(SourceObject ? SourceObject.GetName() : "None" ));
	}
}

void SAudioPropertiesSheetBuilderWidget::SetSourceObject(TObjectPtr<const UObject> NewSourceObject)
{
	if (NewSourceObject == SourceObject)
	{
		return;
	}
	
	UClass* OldSourceClass = SourceObject ? SourceObject->GetClass() : nullptr;
	UClass* NewSourceClass = NewSourceObject->GetClass();
	
	bSourceIsPropertySheet = NewSourceClass ? NewSourceClass->IsChildOf<UAudioPropertiesSheetAsset>() : false;

	if (!bSourceIsPropertySheet && OldSourceClass == NewSourceClass)
	{
		SourceObject = NewSourceObject;
		UpdateSourcePickerComboText();
		if (PropertyRequestsView.IsValid())
		{
			PropertyRequestsView->RebuildList();
		}
		
		//needed to force property columns to show the new values
		FSlateApplication::Get().GetRenderer()->FlushCommands();
		
		return;
	}

	const bool bShowDialog = OldSourceClass != nullptr;

	if (bShowDialog)
	{
		FText Title = FText::FromString("Changing Source Object Class");
		FText Message = FText::FromString("Changing Source Object Class will clear your property selection. Continue?");

		EAppReturnType::Type DialogResponse = FMessageDialog::Open(EAppMsgType::YesNo, Message, Title);

		if (DialogResponse == EAppReturnType::No)
		{
			return;
		}
	}
	
	SourceObject = NewSourceObject;
	UpdateSourcePickerComboText();
	UpdatePropertyRequests();
}

void SAudioPropertiesSheetBuilderWidget::SetSourceAsset(const FAssetData& InSelectedSource)
{
	UObject* NewSourceObject = InSelectedSource.GetAsset();
	SetSourceObject(NewSourceObject);

	if (SourcePickerComboButton)
	{
		SourcePickerComboButton->SetIsOpen(false);
	}
}

void SAudioPropertiesSheetBuilderWidget::SetParentAsset(const FAssetData& InSelectedParent)
{
	UClass* AssetClass = InSelectedParent.GetClass();
	
	if (!ensureAlwaysMsgf(AssetClass && AssetClass->IsChildOf(UAudioPropertiesSheetAsset::StaticClass()), TEXT("Asset class needs to be a child of UAudioPropertiesSheetAsset to be set as parent")))
	{
		return;
	}
	
	GeneratedSheetParent = InSelectedParent;
	UpdateParentPickerComboText();

	if (ParentPickerComboButton)
	{
		ParentPickerComboButton->SetIsOpen(false);
	}
}

TSharedRef<ITableRow> SAudioPropertiesSheetBuilderWidget::OnGenerateRowForList(
    AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr InItem,
    const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(AudioPropertiesSheetBuilderWidget::SPropertyRow, OwnerTable, InItem, SourceObject.Get());
}

FReply SAudioPropertiesSheetBuilderWidget::OnMakePropertySheetClicked()
{
	TObjectPtr<UAudioPropertiesSheetAsset> ParentPtr = nullptr;

	if (GeneratedSheetParent.IsValid())
	{
		ParentPtr = Cast<UAudioPropertiesSheetAsset>(GeneratedSheetParent.GetAsset());
	}

	AudioPropertiesSheetAssetBuilder::ESourceObjectParsingType ParsingType = bSourceIsPropertySheet ? AudioPropertiesSheetAssetBuilder::ESourceObjectParsingType::PropertySheet : AudioPropertiesSheetAssetBuilder::ESourceObjectParsingType::UObject;
	
	FAudioPropertiesSheetAssetBuilder::BuildPropertySheetFromPropertyDataArray(SourceObject, ParentPtr, PropertyRequests, ParsingType);
		
	return FReply::Handled();
}

void SAudioPropertiesSheetBuilderWidget::OnSearchTextChanged(const FText& SearchText)
{
	const FString SearchString = SearchText.ToString();
	FilteredPropertyRequests.Empty();

	for (const AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr& PropertyRequest : PropertyRequests)
	{
		const FProperty* Property = PropertyRequest->CachedProperty;

		if (!Property)
		{
			continue;
		}

		if (Property->GetFName().ToString().Contains(SearchString) || Property->GetDisplayNameText().ToString().Contains(SearchString))
		{
			FilteredPropertyRequests.Add(PropertyRequest);
		}
	}
	
	SortProperties();
}

void SAudioPropertiesSheetBuilderWidget::OnSourceBrowseToButtonClicked() const
{
	FSoftObjectPath AssetPath = SourceObject.GetPath();

	if (AssetPath.IsValid())
	{
		// Find asset data to avoid loading the object
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(AssetPath);
		GEditor->SyncBrowserToObject(AssetData);
	}
}

void SAudioPropertiesSheetBuilderWidget::OnSourceEditButtonClicked() const
{
	GEditor->EditObject(const_cast<UObject*>(SourceObject.Get()));
}

void SAudioPropertiesSheetBuilderWidget::OnSourceUseSelectedButtonClicked()
{
	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);

	if (SelectedAssets.IsEmpty())
	{
		return;
	}

	SetSourceAsset(SelectedAssets[0]);
}

void SAudioPropertiesSheetBuilderWidget::OnParentBrowseToButtonClicked() const
{
	GEditor->SyncBrowserToObject(GeneratedSheetParent);
}

void SAudioPropertiesSheetBuilderWidget::OnParentEditButtonClicked() const
{
	GEditor->EditObject(GeneratedSheetParent.GetAsset());
}

void SAudioPropertiesSheetBuilderWidget::OnParentUseSelectedButtonClicked()
{
	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);

	if (SelectedAssets.IsEmpty())
	{
		return;
	}

	SetSourceAsset(SelectedAssets[0]);
}

EColumnSortMode::Type SAudioPropertiesSheetBuilderWidget::GetColumnSortMode(const FName InColumnName) const
{
	if ( SortedColumn == InColumnName )
	{
		return SortMode;
	}

	return EColumnSortMode::None;
}

void SAudioPropertiesSheetBuilderWidget::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnName, const EColumnSortMode::Type InSortMode)
{
	SortedColumn = ColumnName;
	SortMode = InSortMode;
	SortProperties();
}

void SAudioPropertiesSheetBuilderWidget::SortProperties()
{
	if ( SortMode != EColumnSortMode::None )
	{
		const bool bAscending = EColumnSortMode::Ascending ==  SortMode;
		if ( SortedColumn == AudioPropertiesSheetBuilderWidget::PropertyNameColumnName )
		{
			FilteredPropertyRequests.StableSort( [bAscending](const AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr& First, const AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr& Second)
					{
						if(First->CachedProperty && Second->CachedProperty)
						{
							return First->CachedProperty->GetName() < Second->CachedProperty->GetName() == bAscending;
						}
						return false; 
					}
				);
		}
		else if (SortedColumn == AudioPropertiesSheetBuilderWidget::AddPropertyColumnName )
		{
			FilteredPropertyRequests.StableSort( [bAscending](const AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr& First, const AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr& Second)
				{
					return ((First->bInheritValue + First->bInheritProperty) > (Second->bInheritValue + Second->bInheritProperty)) == bAscending;
				}
			);
		}
		else if (SortedColumn == AudioPropertiesSheetBuilderWidget::CopyPropertyValueColumnName )
		{
			FilteredPropertyRequests.StableSort( [bAscending](const AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr& First, const AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr& Second)
				{
					return ((First->bInheritValue + First->bInheritProperty) > (Second->bInheritValue + Second->bInheritProperty)) == bAscending;
				}
			);
		}
		else
		{
			ensureAlwaysMsgf(false, TEXT("Trying to sort an unsupported column"));
		}
	}

	if (PropertyRequestsView.IsValid())
	{
		PropertyRequestsView->RequestListRefresh();
	}
	
}

FAssetData SAudioPropertiesSheetBuilderWidget::GetSourceAssetData()
{
	if (ensure(SourceObject))
	{
		const FString ObjectPath = SourceObject->GetPathName();
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		return AssetRegistryModule.Get().GetAssetByObjectPath(ObjectPath);
	}

	return FAssetData();
}

#undef LOCTEXT_NAMESPACE