// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "AudioPropertiesSheetAssetBuilder.h"
#include "UObject/SoftObjectPtr.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class SComboButton;
class STextBlock;
class UObject;
class UClass;

class SAudioPropertiesSheetBuilderWidget : public SCompoundWidget
{
public:
    SLATE_BEGIN_ARGS(SAudioPropertiesSheetBuilderWidget) {}
	    
	    //The object to inherit properties from 
		SLATE_ARGUMENT(TObjectPtr<const UObject>, SourceObject)

	    //If true, sets the source object as the parent on widget construction
    	SLATE_ARGUMENT(bool, bSourceIsParent)
    SLATE_END_ARGS()

    void Construct(const FArguments& InArgs);

private:
	TSharedRef<SWidget> GenerateSourceObjectWidget();
    TSharedRef<SWidget> GeneratePropertyListWidget();
	TSharedRef<SWidget> GenerateParentPickerWidget();
	void UpdateSourcePickerComboText() const;
	void UpdateParentPickerComboText() const;
	
	void SetSourceObject(TObjectPtr<const UObject> NewSourceObject);
	void SetSourceAsset(const FAssetData& InSelectedSource);
	void SetParentAsset(const FAssetData& InSelectedParent);
	void UpdatePropertyRequests();

	FAssetData GetSourceAssetData();

	//Widgets callbacks
	TSharedRef<ITableRow> OnGenerateRowForList(AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);
	FReply OnMakePropertySheetClicked();
	void OnSearchTextChanged(const FText& SearchText);

	//Asset browsing delegates - source
	void OnSourceBrowseToButtonClicked() const;
	void OnSourceEditButtonClicked() const;
	void OnSourceUseSelectedButtonClicked();

	//Asset browsing delegates - parent
	void OnParentBrowseToButtonClicked() const;
	void OnParentEditButtonClicked() const;
	void OnParentUseSelectedButtonClicked();

	//Sorting methods
    EColumnSortMode::Type GetColumnSortMode(FName InColumnName) const;
    void OnColumnSortModeChanged(EColumnSortPriority::Type SortPriority, const FName& ColumnName, EColumnSortMode::Type InSortMode);
    void SortProperties();

	TObjectPtr<const UObject> SourceObject;
	FAssetData GeneratedSheetParent;
	bool bSourceIsPropertySheet = false;
	
    TSharedPtr<SListView<AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr>> PropertyRequestsView;
	TArray<AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr> PropertyRequests;
	TArray<AudioPropertiesSheetAssetBuilder::FPropertyRequestPtr> FilteredPropertyRequests;
	
	FName SortedColumn;
	EColumnSortMode::Type SortMode;
	
	TSharedPtr<SComboButton> SourcePickerComboButton;
	TSharedPtr<STextBlock> SourcePickerComboText;

	TSharedPtr<SComboButton> ParentPickerComboButton;
	TSharedPtr<STextBlock> ParentPickerComboText;
};