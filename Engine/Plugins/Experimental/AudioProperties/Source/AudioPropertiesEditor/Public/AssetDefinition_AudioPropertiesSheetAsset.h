// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "AudioPropertiesEditorModule.h"
#include "AudioPropertiesSheet.h"

#include "AssetDefinition_AudioPropertiesSheetAsset.generated.h"

struct FToolMenuSection;

UCLASS(MinimalAPI)
class UAssetDefinition_AudioPropertiesSheetAsset : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()

public:
	//~ Begin UAssetDefinition
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AudioPropertiesSheetAsset", "Audio Properties Sheet"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(AudioPropertiesEditorModule::AssetColor); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAudioPropertiesSheetAsset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	//~ End UAssetDefinition

	FToolMenuSection* FindContextMenuSection(FName SectionName) const;
	
private:
	virtual TArray<FToolMenuSection*> RebuildContextMenuSections() const;
	
};
