// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"

#include "AudioPropertiesBindingsFactory.generated.h"

UCLASS(hidecategories = Object, MinimalAPI)
class UAudioPropertiesBindingsFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};
