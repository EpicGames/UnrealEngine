// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioPropertiesDetailsInjector.h"
#include "Delegates/IDelegateInstance.h"
#include "UObject/ObjectKey.h"

#define UE_API AUDIOPROPERTIESEDITOR_API

class IDetailLayoutBuilder;
class IPropertyHandle;

/**
 * Helper to customize the IDetailCustomization of a class receiving properties from an AudioPropertySheetAsset
 */
class FAudioPropertiesDetailsInjector : public IAudioPropertiesDetailsInjector
{
public:
	/** To be called from IDetailCustomization::CustomizeDetails to customize widgets of injected properties 
	*	@param DetailBuilder					DetailBuilder from CustomizeDetails.
	*	@param PropertySheetPropertyHandle		Property Handle of the property sheet injecting properties on the target UObject. 
	*/	
	UE_API void CustomizeInjectedPropertiesDetails(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> PropertySheetPropertyHandle) override;

	/** To be called from IDetailCustomization::CustomizeDetails to bind the detail view to changes on the property sheet
	*	@param DetailBuilder					DetailBuilder from CustomizeDetails.
	*	@param PropertySheetPropertyHandle		Property Handle of the property sheet injecting properties on the target UObject. 
	*/
	UE_API void BindDetailCustomizationToPropertySheetChanges(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> PropertySheetPropertyHandle) override;

	/** Call this on destruction/pending delete of the details customization to unbind the detail view to changes on the property sheet */
	UE_API void UnbindFromPropertySheetChanges() override;

private:
	UE_API void BindDetailsRefreshToPropertySheetSwaps(const IDetailLayoutBuilder& DetailBuilder, TArray<TWeakObjectPtr<UObject>>& ObjectsBeingCustomized, TSharedRef<IPropertyHandle> PropertySheetPropertyHandle);
	UE_API void BindDetailsRefreshToPropertySheetChanges(IDetailLayoutBuilder& DetailBuilder, TArray<TWeakObjectPtr<UObject>>& ObjectsBeingCustomized, TSharedRef<IPropertyHandle> PropertySheetPropertyHandle);

	UE_API void CreateOverriddenPropertiesWidgets(IDetailLayoutBuilder& DetailBuilder, TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized, TSharedRef<IPropertyHandle> PropertySheetPropertyHandle);

	TArray<FDelegateHandle> OnTargetObjectPropertyChangeDelegateHandles;
	TMultiMap<FObjectKey, FDelegateHandle> InjectedPropertiesBindings;
};

/**
 * Modular feature builder to instantiate an FAudioPropertiesDetailsInjector
 */
class FAudioPropertiesDetailsInjectorBuilder : public IAudioPropertiesDetailsInjectorBuilder
{
public:
	virtual IAudioPropertiesDetailsInjector* CreateAudioPropertiesDetailsInjector() override {return new FAudioPropertiesDetailsInjector; }
};

#undef UE_API
