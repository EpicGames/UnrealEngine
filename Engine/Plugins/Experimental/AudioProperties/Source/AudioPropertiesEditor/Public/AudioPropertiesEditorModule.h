// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class IAudioPropertiesDetailsInjector;
class FAudioPropertiesSheetBuilderInstantiator;
class FAudioPropertiesDetailsInjectorBuilder;

namespace AudioPropertiesEditorModule
{
	inline constexpr FColor AssetColor = FColor(201, 29, 85);
}

class FAudioPropertiesEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TSharedPtr<FAudioPropertiesSheetBuilderInstantiator> PropertiesSheetBuilder = nullptr;
	
	TSharedPtr<FAudioPropertiesDetailsInjectorBuilder> DetailsInjectorBuilder = nullptr;
};