// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "InstancedStructDetails.h"
#include "PropertyBagDetails.h"

struct FAudioPropertiesSheet;
class UAudioPropertiesSheetAsset;


class FInstancedAudioPropertyBagDetails : public FPropertyBagInstanceDataDetails
{
public:

	FInstancedAudioPropertyBagDetails(
		TSharedPtr<IPropertyHandle> InInstancedPropertyBagHandle,
		const TSharedPtr<IPropertyUtilities>& InPropUtils,
		const UAudioPropertiesSheetAsset* InOwningAudioPropertiesSheetAsset,
		FAudioPropertiesSheet* InLeafMostPropertySheet, 
		bool bInIsLeafBag = false);

	virtual void OnChildRowAdded(IDetailPropertyRow& ChildRow)override;

	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override {}

private:
	void DrawLeafBagChildRow(IDetailPropertyRow& ChildRow);
	void DrawInheritedBagChildRow(IDetailPropertyRow& ChildRow);

	void HandlePropertyOverrideChanged(TSharedPtr<IPropertyHandle> PropertyThatChanged, const bool bOverrideState);
	
	TSharedPtr<IPropertyHandle> InstancedPropertyBagHandle = nullptr;
	const UAudioPropertiesSheetAsset* OwningAudioPropertiesSheetAsset = nullptr;
	FAudioPropertiesSheet* LeafMostPropertySheet = nullptr;
	TSharedPtr<IPropertyUtilities> CachedPropUtils;
	bool bIsLeafBag = false;
};